{
	"FileVersion" : 3,
	"Version" : 0.1,
	"VersionName" : "0.1",
	"FriendlyName" : "OpenColorIO (OCIO)",
	"Description" : "Provides support for OpenColorIO",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "OpenColorIO",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OpenColorIOEditor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"Plugins":
	[
	]
}
