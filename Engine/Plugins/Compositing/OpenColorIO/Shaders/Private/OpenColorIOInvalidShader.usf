// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 OpenColorIOErrorShader.usf: Fallback OCIO shader when there is an error with the transform.
 =============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/MiniFontCommon.ush"

Texture2D		InputTexture;
SamplerState	InputTextureSampler;

void MainPS(in noperspective float4 InUVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const int2 PixelPos = (int2)SvPosition.xy % int2(256, 256); // Tile message every 256px
	const float3 TextColor = float3(1, 0.75, 0.25);
	int2 Cursor = int2(128, 128);

	OutColor = InputTexture.Sample(InputTextureSampler, InUVAndScreenPos.xy);

#define P(x)  PrintCharacter(PixelPos, OutColor.xyz, TextColor, Cursor, x);

	P(_O_)P(_C_)P(_I_)P(_O_)
	P(_SPC_)
	P(_I_)P(_N_)P(_V_)P(_A_)P(_L_)P(_I_)P(_D_)
}