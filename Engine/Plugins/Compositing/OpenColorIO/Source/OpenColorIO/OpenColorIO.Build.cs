// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	using System.IO;

	public class OpenColorIO : ModuleRules
	{
		public OpenColorIO(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Projects",
					"RHI",
					"RenderCore",
					"Renderer",
					"ImageCore",
					"Slate",
					"SlateCore",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings",
					"OpenColorIOWrapper"
				});

			if (Target.bBuildWithEditorOnlyData)
			{
				PrivateDependencyModuleNames.Add("DerivedDataCache");
			}

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"TargetPlatform",
						"EditorFramework",
						"UnrealEd"
					});
			}
		}
	}
}
