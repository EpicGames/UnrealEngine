// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOBlueprintLibrary.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "OpenColorIORendering.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOBlueprintLibrary)


UOpenColorIOBlueprintLibrary::UOpenColorIOBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


bool UOpenColorIOBlueprintLibrary::ApplyColorSpaceTransform(const UObject* WorldContextObject, const FOpenColorIOColorConversionSettings& ConversionSettings, UTexture* InputTexture, UTextureRenderTarget2D* OutputRenderTarget)
{
	return FOpenColorIORendering::ApplyColorTransform(WorldContextObject->GetWorld(), ConversionSettings, InputTexture, OutputRenderTarget);
}

