// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOColorSpace.h"

#include "OpenColorIOConfiguration.h"
#include "OpenColorIOSettings.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOColorSpace)

/*
 * FOpenColorIOColorSpace implementation
 */

const TCHAR* FOpenColorIOColorSpace::FamilyDelimiter = TEXT("/");

FOpenColorIOColorSpace::FOpenColorIOColorSpace(const FString& InColorSpaceName, int32 InColorSpaceIndex, const FString& InFamilyName, const FString& InDescription)
	: ColorSpaceName(InColorSpaceName)
	, FamilyName(InFamilyName)
	, Description(InDescription)
{ }

FString FOpenColorIOColorSpace::ToString() const
{
	if (IsValid())
	{
		return ColorSpaceName;
	}
	return {};
}

bool FOpenColorIOColorSpace::IsValid() const
{
	return !ColorSpaceName.IsEmpty();
}

void FOpenColorIOColorSpace::Reset()
{
	ColorSpaceName.Reset();
	FamilyName.Reset();
	Description.Reset();
}

FString FOpenColorIOColorSpace::GetFamilyNameAtDepth(int32 InDepth) const
{
	FString ReturnName;

	TArray<FString> Families;
	FamilyName.ParseIntoArray(Families, FamilyDelimiter);
	if (Families.IsValidIndex(InDepth))
	{
		ReturnName = Families[InDepth];
	}
	else
	{
		//No separator found, does it want the first family?
		if (InDepth == 0 && !FamilyName.IsEmpty())
		{
			ReturnName = FamilyName;
		}
	}

	return ReturnName;
}

/*
 * FOpenColorIODisplayView implementation
 */

FOpenColorIODisplayView::FOpenColorIODisplayView(FStringView InDisplayName, FStringView InViewName, const FString& InDescription)
	: Display(InDisplayName)
	, View(InViewName)
	, Description(InDescription)
{
}

FString FOpenColorIODisplayView::ToString() const
{
	if (IsValid())
	{
		return FString::Printf(TEXT("%s - %s"), *Display, *View);
	}

	return {};
}

bool FOpenColorIODisplayView::IsValid() const
{
	return !Display.IsEmpty() && !View.IsEmpty();
}

void FOpenColorIODisplayView::Reset()
{
	Display.Reset();
	View.Reset();
	Description.Reset();
}

/*
 * FOpenColorIOColorConversionSettings implementation
 */

FOpenColorIOColorConversionSettings::FOpenColorIOColorConversionSettings()
	: ConfigurationSource(nullptr)
	, SourceColorSpace()
	, DestinationColorSpace()
	, DestinationDisplayView()
	, DisplayViewDirection(EOpenColorIOViewTransformDirection::Forward)
{

}

void FOpenColorIOColorConversionSettings::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (!GetDefault<UOpenColorIOSettings>()->bSupportInverseViewTransforms)
		{
			// Enforce forward direction only
			DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;
		}
	}
#endif
}

FString FOpenColorIOColorConversionSettings::ToString() const
{
	if (::IsValid(ConfigurationSource))
	{
		if (IsDisplayView())
		{
			switch (DisplayViewDirection)
			{
			case EOpenColorIOViewTransformDirection::Forward:
				return FString::Printf(TEXT("%s config - %s to %s"), *ConfigurationSource->GetName(), *SourceColorSpace.ToString(), *DestinationDisplayView.ToString());
			case EOpenColorIOViewTransformDirection::Inverse:
				return FString::Printf(TEXT("%s config - %s to %s"), *ConfigurationSource->GetName(), *DestinationDisplayView.ToString(), *SourceColorSpace.ToString());
			default:
				checkNoEntry();
				return FString();
			}
		}
		else
		{
			return FString::Printf(TEXT("%s config - %s to %s"), *ConfigurationSource->GetName(), *SourceColorSpace.ToString(), *DestinationColorSpace.ToString());
		}
	}
	return TEXT("<Invalid Conversion>");
}

bool FOpenColorIOColorConversionSettings::IsValid() const
{
	if (::IsValid(ConfigurationSource))
	{
		if (IsDisplayView())
		{
			if (SourceColorSpace.IsValid() && DestinationDisplayView.IsValid())
			{
				return ConfigurationSource->HasTransform(SourceColorSpace.ColorSpaceName, DestinationDisplayView.Display, DestinationDisplayView.View, DisplayViewDirection);
			}
		}
		else
		{
			if (SourceColorSpace.IsValid() && DestinationColorSpace.IsValid())
			{
				return ConfigurationSource->HasTransform(SourceColorSpace.ColorSpaceName, DestinationColorSpace.ColorSpaceName);
			}
		}
	}

	return false;
}

FString FOpenColorIOColorConversionSettings::GetSourceString() const
{
	if (SourceColorSpace.IsValid())
	{
		return SourceColorSpace.ToString();
	}

	return {};
}

FString FOpenColorIOColorConversionSettings::GetDestinationString() const
{
	if (IsDisplayView() && DestinationDisplayView.IsValid())
	{
		return DestinationDisplayView.ToString();
	}
	else if (DestinationColorSpace.IsValid())
	{
		return DestinationColorSpace.ToString();
	}

	return {};
}

void FOpenColorIOColorConversionSettings::Reset(bool bResetConfigurationSource)
{
	if (bResetConfigurationSource)
	{
		ConfigurationSource = nullptr;
	}
	
	SourceColorSpace.Reset();
	DestinationColorSpace.Reset();
	DestinationDisplayView.Reset();
	DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;
}

void FOpenColorIOColorConversionSettings::ValidateColorSpaces()
{
	if (::IsValid(ConfigurationSource))
	{
		if (!ConfigurationSource->HasDesiredColorSpace(SourceColorSpace))
		{
			SourceColorSpace.Reset();
		}
		if (!ConfigurationSource->HasDesiredColorSpace(DestinationColorSpace))
		{
			DestinationColorSpace.Reset();
		}
		if (!ConfigurationSource->HasDesiredDisplayView(DestinationDisplayView))
		{
			DestinationDisplayView.Reset();
		}
	}
	else
	{
		Reset();
	}
}

bool FOpenColorIOColorConversionSettings::IsDisplayView() const
{
	return DestinationDisplayView.IsValid();
}

bool FOpenColorIODisplayConfiguration::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	
	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FOpenColorIODisplayConfiguration::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::OpenColorIODisabledDisplayConfigurationDefault)
		{
			// Retain previous behavior: enabled only when the settings were valid
			if (ColorConfiguration.IsValid())
			{
				bIsEnabled = true;
			}
		}
	}
}
