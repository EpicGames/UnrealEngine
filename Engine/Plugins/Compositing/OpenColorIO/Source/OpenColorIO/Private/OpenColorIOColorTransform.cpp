// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOColorTransform.h"

#include "Containers/SortedMap.h"
#include "Materials/MaterialInterface.h"
#include "Math/PackedVector.h"
#include "Modules/ModuleManager.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOModule.h"
#include "OpenColorIOSettings.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOColorTransform)


#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "Editor.h"
#endif //WITH_EDITOR

#include "Interfaces/ITargetPlatform.h"
#include "OpenColorIOWrapper.h"
#include "OpenColorIODerivedDataVersion.h"
#include "OpenColorIOShader.h"

namespace
{
	// Returns the (native) config wrapper from the configuration object.
	FOpenColorIOWrapperConfig* GetTransformConfigWrapper(UOpenColorIOConfiguration* InConfigurationOwner)
	{
		if (IsValid(InConfigurationOwner))
		{
#if WITH_EDITOR
			return InConfigurationOwner->GetConfigWrapper();
#else
			// In non-editor modes, we don't automatically load the config so it needs to be lazily created here.
			return InConfigurationOwner->GetOrCreateConfigWrapper();
#endif
		}

		return nullptr;
	}
}

static TAutoConsoleVariable<bool> CVarOpenColorIOAllowAsyncCompilation(
	TEXT("OpenColorIO.AllowAsyncCompilation"),
	true,
	TEXT("When enabled, we enforce synchronous shader compilation.")
);

void UOpenColorIOColorTransform::SerializeOpenColorIOShaderMaps(const TMap<const ITargetPlatform*, TArray<FOpenColorIOTransformResource*>>* PlatformColorTransformResourcesToSavePtr, FArchive& Ar, TArray<FOpenColorIOTransformResource>&  OutLoadedResources)
{
	if (Ar.IsSaving())
	{
		int32 NumResourcesToSave = 0;
		const TArray<FOpenColorIOTransformResource*>* ColorTransformResourcesToSavePtr = nullptr;
		if (Ar.IsCooking())
		{
			check(PlatformColorTransformResourcesToSavePtr);
			auto& PlatformColorTransformResourcesToSave = *PlatformColorTransformResourcesToSavePtr;

			ColorTransformResourcesToSavePtr = PlatformColorTransformResourcesToSave.Find(Ar.CookingTarget());
			check(ColorTransformResourcesToSavePtr != nullptr || (Ar.GetLinker() == nullptr));
			if (ColorTransformResourcesToSavePtr != nullptr)
			{
				NumResourcesToSave = ColorTransformResourcesToSavePtr->Num();
			}
		}

		Ar << NumResourcesToSave;

		if (ColorTransformResourcesToSavePtr)
		{
			const TArray<FOpenColorIOTransformResource*> &ColorTransformResourcesToSave = *ColorTransformResourcesToSavePtr;
			for (int32 ResourceIndex = 0; ResourceIndex < NumResourcesToSave; ResourceIndex++)
			{
				ColorTransformResourcesToSave[ResourceIndex]->SerializeShaderMap(Ar);
			}
		}

	}
	else if (Ar.IsLoading())
	{
		int32 NumLoadedResources = 0;
		Ar << NumLoadedResources;
		OutLoadedResources.Empty(NumLoadedResources);

		for (int32 ResourceIndex = 0; ResourceIndex < NumLoadedResources; ResourceIndex++)
		{
			FOpenColorIOTransformResource& LoadedResource = OutLoadedResources.AddDefaulted_GetRef();
			LoadedResource.SerializeShaderMap(Ar);
		}
	}
}

void UOpenColorIOColorTransform::ProcessSerializedShaderMaps(UOpenColorIOColorTransform* Owner, TArray<FOpenColorIOTransformResource>& LoadedResources, FOpenColorIOTransformResource* (&OutColorTransformResourcesLoaded)[ERHIFeatureLevel::Num])
{
	check(IsInGameThread());

	for (int32 ResourceIndex = 0; ResourceIndex < LoadedResources.Num(); ResourceIndex++)
	{
		FOpenColorIOTransformResource& LoadedResource = LoadedResources[ResourceIndex];
		FOpenColorIOShaderMap* LoadedShaderMap = LoadedResource.GetGameThreadShaderMap();

		if (LoadedShaderMap && LoadedShaderMap->GetShaderPlatform() == GMaxRHIShaderPlatform)
		{
			ERHIFeatureLevel::Type LoadedFeatureLevel = LoadedShaderMap->GetShaderMapId().FeatureLevel;
			if (!OutColorTransformResourcesLoaded[LoadedFeatureLevel])
			{
				OutColorTransformResourcesLoaded[LoadedFeatureLevel] = new FOpenColorIOTransformResource();
			}

			OutColorTransformResourcesLoaded[LoadedFeatureLevel]->SetInlineShaderMap(LoadedShaderMap);
		}
	}
}

void UOpenColorIOColorTransform::GetOpenColorIOLUTKeyGuid(const FString& InProcessorIdentifier, const FName& InName, FGuid& OutLutGuid)
{
#if WITH_EDITOR
	FString DDCKey = FDerivedDataCacheInterface::BuildCacheKey(TEXT("OCIOLUT"), OPENCOLORIO_DERIVEDDATA_VER, *InProcessorIdentifier);

	//Keep library version in the DDC key to invalidate it once we move to a new library
	DDCKey += TEXT("OCIOVersion");
	DDCKey += OpenColorIOWrapper::GetVersion();

	if (!InName.IsNone())
	{
		DDCKey += InName.ToString();
	}

	const uint32 KeyLength = DDCKey.Len() * sizeof(DDCKey[0]);
	uint32 Hash[5];
	FSHA1::HashBuffer(*DDCKey, KeyLength, reinterpret_cast<uint8*>(Hash));
	OutLutGuid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
#endif
}

UOpenColorIOColorTransform::UOpenColorIOColorTransform(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UOpenColorIOColorTransform::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

#if WITH_EDITOR
	SerializeOpenColorIOShaderMaps(&CachedColorTransformResourcesForCooking, Ar, LoadedTransformResources);
#else
	SerializeOpenColorIOShaderMaps(nullptr, Ar, LoadedTransformResources);
#endif
	
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::OpenColorIOAssetCacheSerialization)
	{
		if (Ar.IsLoading())
		{
			// The asset previously always saved the number of LUTs, and conditionally the textures when cooked.
			// We can safely ignore the cooked case since the current logic will be applied. However, we must
			// still pretend to read the number of textures for the archive load to match the expected size.

			int32 NumTexturesDeprecated = 0;
			Ar << NumTexturesDeprecated;
		}
	}
}

bool UOpenColorIOColorTransform::Initialize(const FString& InSourceColorSpace, const FString& InDestinationColorSpace)
{

	if (InSourceColorSpace.IsEmpty() || InDestinationColorSpace.IsEmpty())
	{
		return false;
	}

	SourceColorSpace = InSourceColorSpace;
	DestinationColorSpace = InDestinationColorSpace;
	bIsDisplayViewType = false;

#if WITH_EDITOR
	ProcessTransformForGPU();
#endif
	FlushResourceShaderMaps();
	CacheResourceShadersForRendering();

	return true;
}

bool UOpenColorIOColorTransform::Initialize(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection)
{
	if (InSourceColorSpace.IsEmpty() || InDisplay.IsEmpty() || InView.IsEmpty())
	{
		return false;
	}

	SourceColorSpace = InSourceColorSpace;
	DestinationColorSpace.Empty();
	Display = InDisplay;
	View = InView;
	bIsDisplayViewType = true;
	DisplayViewDirection = InDirection;

#if WITH_EDITOR
	ProcessTransformForGPU();
#endif
	FlushResourceShaderMaps();
	CacheResourceShadersForRendering();

	return true;
}

#if WITH_EDITOR
void UOpenColorIOColorTransform::CacheResourceShadersForCooking(EShaderPlatform InShaderPlatform, const ITargetPlatform* TargetPlatform, const FString& InShaderHash, const FString& InShaderCode, const FString& InRawConfigHash, TArray<FOpenColorIOTransformResource*>& OutCachedResources)
{
	const ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(InShaderPlatform);

	FOpenColorIOTransformResource* NewResource = new FOpenColorIOTransformResource();

	FName AssetPath = GetOutermost()->GetFName();
	NewResource->SetupResource((ERHIFeatureLevel::Type)TargetFeatureLevel, InShaderHash, InShaderCode, InRawConfigHash, GetTransformFriendlyName(), AssetPath, BuildShaderParamMetadata());

	const bool bApplyCompletedShaderMap = false;
	const bool bIsCooking = true;
	CacheShadersForResources(InShaderPlatform, NewResource, bApplyCompletedShaderMap, bIsCooking);

	OutCachedResources.Add(NewResource);
}

void UOpenColorIOColorTransform::ProcessTransformForGPU()
{
#if WITH_OCIO
	FOpenColorIOWrapperProcessor TransformProcessor;
	if (GetTransformProcessor(TransformProcessor))
	{
		// Note for a ticket in jira [UE-202180]
		// When the Textures array is emptied, this will cause all existing UTexture objects in the game thread to be deleted.
		// The resources that were returned earlier using GetRenderResources() and GetRenderPassResources() will also be deleted.
		// Other plugins store OCIO resources in the FOpenColorIORenderPassResources structure, which uses pointers to FTextureResources
		// that are used without reference counting. Once deleted in the game thread, they become invalid in the rendering thread.
		LookupTextures.Reset();

		const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();
		const FOpenColorIOWrapperGPUProcessor GPUProcessor = FOpenColorIOWrapperGPUProcessor(TransformProcessor, Settings->bUseLegacyProcessor);
		const FString GpuProcessorHash = GPUProcessor.GetCacheID();

		// Process 3D luts
		for (uint32 Index = 0; Index < GPUProcessor.GetNum3DTextures(); ++Index)
		{
			TextureFilter Filter;
			FName TextureName;
			uint32 EdgeLength = 0;
			const float* TextureValues = 0x0;

			bool bSuccess = GPUProcessor.Get3DTexture(Index, TextureName, EdgeLength, Filter, TextureValues);
			checkf(bSuccess, TEXT("Failed to read OCIO 3D LUT data."));

			LookupTextures.Add(CreateTexture3DLUT(GpuProcessorHash, TextureName, EdgeLength, Filter, TextureValues));
		}

		// Process 1D luts
		for (uint32 Index = 0; Index < GPUProcessor.GetNumTextures(); ++Index)
		{
			FName TextureName;
			uint32 TextureWidth = 0;
			uint32 TextureHeight = 0;
			TextureFilter Filter;
			bool bRedChannelOnly = false;
			const float* TextureValues = 0x0;

			bool bSuccess = GPUProcessor.GetTexture(Index, TextureName, TextureWidth, TextureHeight, Filter, bRedChannelOnly, TextureValues);
			checkf(bSuccess, TEXT("Failed to read OCIO 1D LUT data."));

			LookupTextures.Add(CreateTexture1DLUT(GpuProcessorHash, TextureName, TextureWidth, TextureHeight, Filter, bRedChannelOnly, TextureValues));
		}
		
		// Generate shader code
		GPUProcessor.GetShader(GeneratedShaderHash, GeneratedShader);
	}
#endif //WITH_OCIO
}

TObjectPtr<UTexture> UOpenColorIOColorTransform::CreateTexture3DLUT(const FString& InProcessorIdentifier, const FName& InName, uint32 InLutLength, TextureFilter InFilter, const float* InSourceData)
{
	check(InSourceData);
	const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();

	/* Note here that while it is possible to create proper 32f textures using [UTexture]::CreateTransient and reparenting
	 * via Texture->Rename(nullptr, this), cooking would fail as it remains unsupported currently for those formats.
	 * (The same note applies to 1D LUT creation.) */

	TObjectPtr<UVolumeTexture> OutTexture = NewObject<UVolumeTexture>(this, InName);

	//Initializes source data with the raw LUT. If it's found in DDC, the resulting platform data will be fetched from there. 
	//If not, the source data will be used to generate the platform data.
	OutTexture->MipGenSettings = TMGS_NoMipmaps;
	OutTexture->SRGB = 0;
	OutTexture->LODGroup = TEXTUREGROUP_ColorLookupTable;
	if (Settings->bUse32fLUT)
	{
		// 32f resources have to be explicitely requested using this compression setting.
		OutTexture->CompressionSettings = TextureCompressionSettings::TC_HDR_F32;
	}
	else
	{
		OutTexture->CompressionNone = true;
	}
	OutTexture->Filter = InFilter;
	OutTexture->AddressMode = TextureAddress::TA_Clamp;
	OutTexture->Source.Init(InLutLength, InLutLength, InLutLength, /*NumMips=*/ 1, TSF_RGBA32F, nullptr);

	FLinearColor* MipData = reinterpret_cast<FLinearColor*>(OutTexture->Source.LockMip(0));
	for (uint32 Z = 0; Z < InLutLength; ++Z)
	{
		for (uint32 Y = 0; Y < InLutLength; Y++)
		{
			FLinearColor* Row = &MipData[Y * InLutLength + Z * InLutLength * InLutLength];
			const float* Source = &InSourceData[Y * InLutLength * 3 + Z * InLutLength * InLutLength * 3];
			for (uint32 X = 0; X < InLutLength; X++)
			{
				FLinearColor& TargetColor = Row[X];
				TargetColor.R = Source[X * 3 + 0];
				TargetColor.G = Source[X * 3 + 1];
				TargetColor.B = Source[X * 3 + 2];
				TargetColor.A = 1.0f;
			}
		}
	}
	OutTexture->Source.UnlockMip(0);

	//Generate a Guid from the identifier received from the library and our DDC version.
	FGuid LutGuid;
	GetOpenColorIOLUTKeyGuid(InProcessorIdentifier, InName, LutGuid);
	OutTexture->Source.SetId(LutGuid, true);

	//Process our new texture to be usable in rendering pipeline.
	OutTexture->UpdateResource();

	return OutTexture;
}

TObjectPtr<UTexture> UOpenColorIOColorTransform::CreateTexture1DLUT(const FString& InProcessorIdentifier, const FName& InName, uint32 InTextureWidth, uint32 InTextureHeight, TextureFilter InFilter, bool bRedChannelOnly, const float* InSourceData)
{
	check(InSourceData);
	const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();

	TObjectPtr<UTexture2D> OutTexture = NewObject<UTexture2D>(this, InName);
	OutTexture->MipGenSettings = TMGS_NoMipmaps;
	OutTexture->SRGB = 0;
	OutTexture->LODGroup = TEXTUREGROUP_ColorLookupTable;
	if (Settings->bUse32fLUT)
	{
		// 32f resources have to be explicitely requested using this compression setting.
		OutTexture->CompressionSettings = bRedChannelOnly ? TextureCompressionSettings::TC_SingleFloat : TextureCompressionSettings::TC_HDR_F32;
	}
	else
	{
		OutTexture->CompressionNone = true;
	}
	OutTexture->Filter = InFilter;
	OutTexture->AddressX = TextureAddress::TA_Clamp;
	OutTexture->AddressY = TextureAddress::TA_Clamp;

	if (bRedChannelOnly)
	{
		OutTexture->Source.Init(InTextureWidth, InTextureHeight, /*NumSlices=*/ 1, /*NumMips=*/ 1, TSF_R32F, reinterpret_cast<const uint8*>(InSourceData));
	}
	else
	{
		OutTexture->Source.Init(InTextureWidth, InTextureHeight, /*NumSlices=*/ 1, /*NumMips=*/ 1, TSF_RGBA32F, nullptr);

		FLinearColor* MipData = reinterpret_cast<FLinearColor*>(OutTexture->Source.LockMip(0));
		for (uint32 Y = 0; Y < InTextureHeight; Y++)
		{
			FLinearColor* Row = &MipData[Y * InTextureWidth];
			const float* Source = &InSourceData[Y * InTextureWidth * 3];
			for (uint32 X = 0; X < InTextureWidth; X++)
			{
				FLinearColor& TargetColor = Row[X];
				TargetColor.R = Source[X * 3 + 0];
				TargetColor.G = Source[X * 3 + 1];
				TargetColor.B = Source[X * 3 + 2];
				TargetColor.A = 1.0f;
			}
		}
		OutTexture->Source.UnlockMip(0);
	}

	//Generate a Guid from the identifier received from the library and our DDC version.
	FGuid LutGuid;
	GetOpenColorIOLUTKeyGuid(InProcessorIdentifier, InName, LutGuid);
	OutTexture->Source.SetId(LutGuid, true);

	//Process our new texture to be usable in rendering pipeline.
	OutTexture->UpdateResource();

	return OutTexture;
}
#endif //WITH_EDITOR

void UOpenColorIOColorTransform::CacheResourceShadersForRendering(bool /*bRegenerateId*/)
{
	if (FApp::CanEverRender())
	{
		const UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>();
		if (IsValid(ConfigurationOwner))
		{
			//OCIO shaders are simple, we should be compatible with any feature levels. Use the levels required for materials.
			uint32 FeatureLevelsToCompile = UMaterialInterface::GetFeatureLevelsToCompileForAllMaterials();
			while (FeatureLevelsToCompile != 0)
			{
				ERHIFeatureLevel::Type CacheFeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
				const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[CacheFeatureLevel];

				FOpenColorIOTransformResource*& TransformResource = ColorTransformResources[CacheFeatureLevel];
				if (TransformResource == nullptr)
				{
					TransformResource = new FOpenColorIOTransformResource();
				}

				FString ShaderCodeHash;
				FString ShaderCode;
				FString RawConfigHash;
				FName AssetPath;
#if WITH_EDITOR
				if (GeneratedShader.IsEmpty())
				{
					continue;
				}

				ShaderCodeHash = GeneratedShaderHash;
				ShaderCode = GeneratedShader;
				RawConfigHash = ConfigurationOwner->GetConfigWrapper()->GetCacheID();
				AssetPath = GetOutermost()->GetFName();
#endif // WITH_EDITOR
				TransformResource->SetupResource(CacheFeatureLevel, ShaderCodeHash, ShaderCode, RawConfigHash, GetTransformFriendlyName(), AssetPath, BuildShaderParamMetadata());

				const bool bApplyCompletedShaderMap = true;

				// If PIE or -game - we don't want to be doing shader cooking asynchronosly.
				bool bIsSynchronous = FApp::IsGame();
				
				CacheShadersForResources(ShaderPlatform, TransformResource, bApplyCompletedShaderMap, bIsSynchronous);
			}
		}
	}
}

void UOpenColorIOColorTransform::CacheShadersForResources(EShaderPlatform InShaderPlatform, FOpenColorIOTransformResource* InResourceToCache, bool bApplyCompletedShaderMapForRendering, bool bIsSynchronous, const ITargetPlatform* TargetPlatform)
{
	if (!CVarOpenColorIOAllowAsyncCompilation.GetValueOnAnyThread())
	{
		bIsSynchronous = true;
	}

	const bool bSuccess = InResourceToCache->CacheShaders(InShaderPlatform, TargetPlatform, bApplyCompletedShaderMapForRendering, bIsSynchronous);

	if (!bSuccess)
	{
		UE_ASSET_LOG(LogOpenColorIO, Warning, this, TEXT("Failed to compile OCIO ColorSpace transform %s shader for platform %s.")
			, *LegacyShaderPlatformToShaderFormat(InShaderPlatform).ToString()
			, *InResourceToCache->GetFriendlyName());

		const TArray<FString>& CompileErrors = InResourceToCache->GetCompileErrors();
		for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
		{
			UE_LOGF(LogOpenColorIO, Warning, "	%ls", *CompileErrors[ErrorIndex]);
		}
	}
}

bool UOpenColorIOColorTransform::GetRenderResources(ERHIFeatureLevel::Type InFeatureLevel, FOpenColorIOTransformResource*& OutShaderResource, TArray<TWeakObjectPtr<UTexture>>& OutTextureResources) const
{
	if (!AreRenderResourcesReady())
	{
		return false;
	}

	OutShaderResource = ColorTransformResources[InFeatureLevel];
	
	if (OutShaderResource)
	{
		OutTextureResources.Reserve(LookupTextures.Num());

		for (const TObjectPtr<UTexture>& Texture : LookupTextures)
		{
			OutTextureResources.Add(Texture);
		}
	}
	
	return OutShaderResource != nullptr;
}

bool UOpenColorIOColorTransform::AreRenderResourcesReady() const
{
	check(IsInGameThread());

	int32 NumShadersRequired = 0;
	int32 NumShadersReady = 0;
	for (const FOpenColorIOTransformResource* Resource : ColorTransformResources)
	{
		if (Resource)
		{
			NumShadersRequired++;

			if (Resource->IsCompilationFinished())
			{
				NumShadersReady++;
			}
		}
	}

	// All of the required shaders should have finished compiling.
	if (NumShadersRequired != NumShadersReady)
	{
		return false;
	}

	// Textures are optional, depending on the transform.
	for (const TObjectPtr<UTexture>& Texture : LookupTextures)
	{
		if (Texture->GetResource() == nullptr)
		{
			return false;
		}

#if WITH_EDITOR
		// Note: this check is valid the first time a texture is created, but wouldn't be relevant if we updated existing textures.
		if (Texture->IsCompiling())
		{
			return false;
		}
#endif
	}

	return true;
}

bool UOpenColorIOColorTransform::IsTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace) const
{
	if (!bIsDisplayViewType)
	{
		return SourceColorSpace == InSourceColorSpace && DestinationColorSpace == InDestinationColorSpace;
	}

	return false;
}

bool UOpenColorIOColorTransform::IsTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection) const
{
	if (bIsDisplayViewType)
	{
		return SourceColorSpace == InSourceColorSpace && Display == InDisplay && View == InView && DisplayViewDirection == InDirection;
	}

	return false;
}

bool UOpenColorIOColorTransform::GetTransformProcessor(FOpenColorIOWrapperProcessor& OutProcessor, const TMap<FString, FString>& InContextOverride) const
{
#if WITH_OCIO
	UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>();
	const FOpenColorIOWrapperConfig* ConfigWrapper = GetTransformConfigWrapper(ConfigurationOwner);

	if (!ConfigWrapper)
	{
		UE_LOGF(LogOpenColorIO, Error, "Failed to create processor for color transform %ls. Configuration file [%ls] was invalid.", *GetTransformFriendlyName(), ConfigurationOwner ? *ConfigurationOwner->GetConfigurationFile().FilePath : TEXT("Missing"));
		return false;
	}

	// Note: Since OpenColorIO caches processors automatically, we can recreate them without significant additional costs.
	EOpenColorIOViewTransformDirection CurrentDisplayViewDirection;
	if (GetDisplayViewDirection(CurrentDisplayViewDirection))
	{
		OutProcessor = FOpenColorIOWrapperProcessor(
			ConfigWrapper,
			SourceColorSpace,
			Display,
			View,
			CurrentDisplayViewDirection == EOpenColorIOViewTransformDirection::Inverse,
			InContextOverride.IsEmpty() ? ConfigurationOwner->Context : InContextOverride
		);
	}
	else
	{
		OutProcessor = FOpenColorIOWrapperProcessor(
			ConfigWrapper,
			SourceColorSpace,
			DestinationColorSpace,
			InContextOverride.IsEmpty() ? ConfigurationOwner->Context : InContextOverride
		);
	}

	return OutProcessor.IsValid();
#else
	return false;
#endif
}

bool UOpenColorIOColorTransform::TransformColor(FLinearColor& InOutColor) const
{
#if WITH_OCIO
	FOpenColorIOWrapperProcessor TransformProcessor;
	if (GetTransformProcessor(TransformProcessor))
	{
		return TransformProcessor.TransformColor(InOutColor);
	}
#endif

	return false;
}

bool UOpenColorIOColorTransform::TransformImage(const FImageView& InOutImage) const
{
#if WITH_OCIO
	FOpenColorIOWrapperProcessor TransformProcessor;
	if (GetTransformProcessor(TransformProcessor))
	{
		return TransformProcessor.TransformImage(InOutImage);
	}
#endif

	return false;
}

bool UOpenColorIOColorTransform::TransformImage(const FImageView& SrcImage, const FImageView& DestImage) const
{
#if WITH_OCIO
	FOpenColorIOWrapperProcessor TransformProcessor;
	if (GetTransformProcessor(TransformProcessor))
	{
		return TransformProcessor.TransformImage(SrcImage, DestImage);
	}
#endif

	return false;
}

bool UOpenColorIOColorTransform::GetDisplayViewDirection(EOpenColorIOViewTransformDirection& OutDirection) const
{
	if (bIsDisplayViewType)
	{
		OutDirection = DisplayViewDirection;
	}

	return bIsDisplayViewType;
}

void UOpenColorIOColorTransform::AllColorTransformsCacheResourceShadersForRendering()
{
	for (TObjectIterator<UOpenColorIOColorTransform> It; It; ++It)
	{
		UOpenColorIOColorTransform* Transform = *It;

		Transform->CacheResourceShadersForRendering();
	}
}

TMap<FString, FString> UOpenColorIOColorTransform::GetContextKeyValues() const
{
	UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>();
	if (IsValid(ConfigurationOwner))
	{
		return ConfigurationOwner->Context;
	}

	return TMap<FString, FString>();
}

FString UOpenColorIOColorTransform::GetTransformFriendlyName() const
{
	if (bIsDisplayViewType)
	{
		switch (DisplayViewDirection)
		{
		case EOpenColorIOViewTransformDirection::Forward:
			return SourceColorSpace + TEXT(" to ") + Display + TEXT(" - ") + View;
		case EOpenColorIOViewTransformDirection::Inverse:
			return Display + TEXT(" - ") + View + TEXT(" to ") + SourceColorSpace;
		default:
			checkNoEntry();
			return FString();
		}
	}
	else
	{
		return SourceColorSpace + TEXT(" to ") + DestinationColorSpace;
	}
}

void UOpenColorIOColorTransform::FlushResourceShaderMaps()
{
	if (FApp::CanEverRender())
	{
		for (int32 Index = 0; Index < ERHIFeatureLevel::Num; Index++)
		{
			if (ColorTransformResources[Index])
			{
				ColorTransformResources[Index]->ReleaseShaderMap();
			}
		}
	}
}

FShaderParametersMetadata* UOpenColorIOColorTransform::BuildShaderParamMetadata()
{
	// We simply use static allocations for resource names, similar to ComputeFramework::AddParamForType.
	static TArray<FString> AllTextureResourceNames;
	static TArray<FString> AllSamplerResourceNames;

	FShaderParametersMetadataBuilder Builder;

	// First, we add our built-in shader paramaters
	Builder.AddIncludedStruct<FOpenColorIOPixelShaderParameters>();

	// Second, we add the dynamic texture & sampler parameters if present in the transform.
	// We always add the texture resource followed by its sampler. The same order is at render-time,
	// in FOpenColorIORendering::AddPass_RenderThread.
	for (const TObjectPtr<UTexture>& Texture : LookupTextures)
	{
		if (IsValid(Texture))
		{
			int32 TexureIndex = AllTextureResourceNames.AddUnique(Texture->GetName());
			int32 SamplerIndex = AllSamplerResourceNames.AddUnique(Texture->GetName() + TEXT("Sampler"));

			Builder.AddTexture(*AllTextureResourceNames[TexureIndex], Texture.IsA(UVolumeTexture::StaticClass()) ? TEXT("TextureCube") : TEXT("Texture2D"));
			Builder.AddSampler(*AllSamplerResourceNames[SamplerIndex], TEXT("SamplerState"));
		}
	}

	return Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT(""));
}

void UOpenColorIOColorTransform::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!Textures_DEPRECATED.IsEmpty())
	{
		for (const TPair<int32, TObjectPtr<UTexture>>& TexturePair : Textures_DEPRECATED)
		{
			LookupTextures.Add(TexturePair.Value);
		}
		Textures_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	if (FApp::CanEverRender())
	{
		ProcessSerializedShaderMaps(this, LoadedTransformResources, ColorTransformResources);
	}
	else
	{
		// Discard all loaded material resources
		for (FOpenColorIOTransformResource& Resource : LoadedTransformResources)
		{
			Resource.DiscardShaderMap();
		}
	}

	UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>();

	//To be able to fetch OCIO data, make sure our config owner has been postloaded.
	if (ConfigurationOwner)
	{
		ConfigurationOwner->ConditionalPostLoad();

		bool bIsTransformActive = false;

		if (bIsDisplayViewType)
		{
			bIsTransformActive = ConfigurationOwner->HasTransform(SourceColorSpace, Display, View, DisplayViewDirection);
		}
		else
		{
			bIsTransformActive = ConfigurationOwner->HasTransform(SourceColorSpace, DestinationColorSpace);
		}
		
		if (bIsTransformActive)
		{
#if WITH_EDITOR
			// Recently saved transforms will have a valid generated shader which we can use as an indication to skip processing on load.
			if (GeneratedShader.IsEmpty())
			{
				ProcessTransformForGPU();
			}
#endif

			CacheResourceShadersForRendering();
		}
		else
		{
			UE_LOGF(LogOpenColorIO, Verbose, "ConfigurationOwner does not contain [%ls], aborting transform load.", *GetTransformFriendlyName());
		}
	}
	else
	{
		UE_LOGF(LogOpenColorIO, Warning, "Outer is not an UOpenColorIOConfiguration. Outer class: %ls, Outer name: %ls. ", *GetOuter()->GetClass()->GetName(), *GetOuter()->GetName());
	}

	// Empty the list of loaded resources, we don't need it anymore
	LoadedTransformResources.Empty();

}

#if WITH_EDITORONLY_DATA
void UOpenColorIOColorTransform::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UTexture2D::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UVolumeTexture::StaticClass()));
}
#endif

void UOpenColorIOColorTransform::BeginDestroy()
{
	Super::BeginDestroy();

	ReleaseFence.BeginFence();
}

bool UOpenColorIOColorTransform::IsReadyForFinishDestroy()
{
	bool bReady = Super::IsReadyForFinishDestroy();

	return bReady && ReleaseFence.IsFenceComplete();
}

void UOpenColorIOColorTransform::FinishDestroy()
{
	ReleaseResources();

	Super::FinishDestroy();
}

#if WITH_EDITOR

bool UOpenColorIOColorTransform::UpdateShaderInfo(FString& OutShaderCodeHash, FString& OutShaderCode, FString& OutRawConfigHash)
{
	const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();
	UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>();
	const FOpenColorIOWrapperConfig* ConfigWrapper = GetTransformConfigWrapper(ConfigurationOwner);

	if (ConfigWrapper != nullptr)
	{
		FOpenColorIOWrapperProcessor TransformProcessor;
		if (GetTransformProcessor(TransformProcessor))
		{
			const FOpenColorIOWrapperGPUProcessor GPUProcessor = FOpenColorIOWrapperGPUProcessor(TransformProcessor, Settings->bUseLegacyProcessor);
			if (GPUProcessor.IsValid())
			{
				OutRawConfigHash = ConfigWrapper->GetCacheID();

				return GPUProcessor.GetShader(OutShaderCodeHash, OutShaderCode);
			}
		}
	}
	else
	{
		UE_LOGF(LogOpenColorIO, Error, "Failed to fetch shader info for color transform %ls. Configuration file [%ls] was invalid.", *GetTransformFriendlyName(), ConfigurationOwner ? *ConfigurationOwner->GetConfigurationFile().FilePath : TEXT("Missing"));
	}

	return false;
}

void UOpenColorIOColorTransform::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	TArray<FName> DesiredShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

	TArray<FOpenColorIOTransformResource*>* CachedColorTransformResourceForPlatformPtr = &CachedColorTransformResourcesForCooking.FindOrAdd(TargetPlatform);

	if (DesiredShaderFormats.Num() > 0)
	{
		FString ShaderCodeHash;
		FString ShaderCode;
		FString RawConfigHash;

		if (GeneratedShader.IsEmpty())
		{
			//Need to re-update shader data when cooking, they may not have been previously fetched.
			UpdateShaderInfo(ShaderCodeHash, ShaderCode, RawConfigHash);
		}
		else if(const UOpenColorIOConfiguration* ConfigurationOwner = GetTypedOuter<UOpenColorIOConfiguration>())
		{
			ShaderCodeHash = GeneratedShaderHash;
			ShaderCode = GeneratedShader;
			RawConfigHash = ConfigurationOwner->GetConfigWrapper()->GetCacheID();
		}

		if (!ShaderCode.IsEmpty())
		{
			// Cache for all the shader formats that the cooking target requires
			for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
			{
				const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
				// Begin caching shaders for the target platform and store the FOpenColorIOTransformResource being compiled into CachedColorTransformResourcesForCooking
				CacheResourceShadersForCooking(LegacyShaderPlatform, TargetPlatform, ShaderCodeHash, ShaderCode, RawConfigHash, *CachedColorTransformResourceForPlatformPtr);
			}
		}
	}
}

bool UOpenColorIOColorTransform::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	const TArray<FOpenColorIOTransformResource*>* CachedColorTransformResourcesForPlatform = CachedColorTransformResourcesForCooking.Find(TargetPlatform);

	if (CachedColorTransformResourcesForPlatform)
	{
		for (const FOpenColorIOTransformResource* const TransformResource : *CachedColorTransformResourcesForPlatform)
		{
			if (TransformResource->IsCompilationFinished() == false)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void UOpenColorIOColorTransform::ClearCachedCookedPlatformData(const ITargetPlatform *TargetPlatform)
{
	TArray<FOpenColorIOTransformResource*>* CachedColorTransformResourcesForPlatform = CachedColorTransformResourcesForCooking.Find(TargetPlatform);
	if (CachedColorTransformResourcesForPlatform != nullptr)
	{
		for (const FOpenColorIOTransformResource* const TransformResource : *CachedColorTransformResourcesForPlatform)
		{
			delete TransformResource;
		}
	}
	CachedColorTransformResourcesForCooking.Remove(TargetPlatform);
}

void UOpenColorIOColorTransform::ClearAllCachedCookedPlatformData()
{
	for (auto It : CachedColorTransformResourcesForCooking)
	{
		TArray<FOpenColorIOTransformResource*>& CachedColorTransformResourcesForPlatform = It.Value;
		for (int32 CachedResourceIndex = 0; CachedResourceIndex < CachedColorTransformResourcesForPlatform.Num(); CachedResourceIndex++)
		{
			delete CachedColorTransformResourcesForPlatform[CachedResourceIndex];
		}
	}

	CachedColorTransformResourcesForCooking.Empty();
}

#endif //WITH_EDITOR

void UOpenColorIOColorTransform::ReleaseResources()
{
	for (int32 FeatureLevelIndex = 0; FeatureLevelIndex < ERHIFeatureLevel::Num; FeatureLevelIndex++)
	{
		FOpenColorIOTransformResource*& CurrentResource = ColorTransformResources[FeatureLevelIndex];
		if (CurrentResource)
		{
			delete CurrentResource;
			CurrentResource = nullptr;
		}
	}

#if WITH_EDITOR
	if (!GExitPurge)
	{
		ClearAllCachedCookedPlatformData();
	}
#endif
}

