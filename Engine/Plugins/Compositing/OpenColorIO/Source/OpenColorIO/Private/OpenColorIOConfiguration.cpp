// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOConfiguration.h"

#include "ColorManagement/ColorSpace.h"
#include "Containers/SortedMap.h"
#include "EngineAnalytics.h"
#include "Engine/VolumeTexture.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Hash/xxhash.h"
#include "Math/PackedVector.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "OpenColorIOColorTransform.h"
#include "OpenColorIOModule.h"
#include "OpenColorIOSettings.h"
#include "OpenColorIOWrapper.h"
#include "TextureResource.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOConfiguration)

#define LOCTEXT_NAMESPACE "OCIOConfiguration"


#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "Interfaces/ITargetPlatform.h"

namespace OCIODirectoryWatcher
{
	/** OCIO supported extensions we should be checking for when something changes in the OCIO config folder. */
	static const TSet<FString> OcioExtensions =
	{
		"spi1d", "spi3d", "3dl", 
		"cc", "ccc", "csp", 
		"cub", "cube", "lut", 
		"mga", "m3d", "spi1d", 
		"spi3d", "spimtx", "vf",
		"ocio"
	};

	static const FName NAME_DirectoryWatcher = "DirectoryWatcher";
}
#endif

UOpenColorIOConfiguration::UOpenColorIOConfiguration(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

void UOpenColorIOConfiguration::BeginDestroy()
{
	StopDirectoryWatch();
	Super::BeginDestroy();
}

bool UOpenColorIOConfiguration::IsTransformReady(const FOpenColorIOColorConversionSettings& InSettings)
{
	TObjectPtr<const UOpenColorIOColorTransform> Transform = FindTransform(InSettings);
	if (IsValid(Transform))
	{
		return Transform->AreRenderResourcesReady();
	}

	return false;
}

bool UOpenColorIOConfiguration::GetRenderResources(ERHIFeatureLevel::Type InFeatureLevel, const FOpenColorIOColorConversionSettings& InSettings, FOpenColorIOTransformResource*& OutShaderResource, TArray<TWeakObjectPtr<UTexture>>& OutTextureResources)
{
	TObjectPtr<const UOpenColorIOColorTransform> Transform = FindTransform(InSettings);
	if (IsValid(Transform))
	{
		return Transform->GetRenderResources(InFeatureLevel, OutShaderResource, OutTextureResources);
	}

	return false;
}

bool UOpenColorIOConfiguration::HasTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace)
{
	TObjectPtr<UOpenColorIOColorTransform>* TransformData = ColorTransforms.FindByPredicate([&](const UOpenColorIOColorTransform* InTransformData)
	{
		return InTransformData->IsTransform(InSourceColorSpace, InDestinationColorSpace);
	});

	return (TransformData != nullptr);
}

bool UOpenColorIOConfiguration::HasTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection)
{
	UE_TRANSITIONAL_OBJECT_PTR(UOpenColorIOColorTransform)* TransformData = ColorTransforms.FindByPredicate([&](const UOpenColorIOColorTransform* InTransformData)
		{
			return InTransformData->IsTransform(InSourceColorSpace, InDisplay, InView, InDirection);
		});

	return (TransformData != nullptr);
}

bool UOpenColorIOConfiguration::HasDesiredColorSpace(const FOpenColorIOColorSpace& ColorSpace) const
{
	if (ColorSpace.IsValid())
	{
		return DesiredColorSpaces.Find(ColorSpace) != INDEX_NONE;
	}

	return false;
}

bool UOpenColorIOConfiguration::HasDesiredDisplayView(const FOpenColorIODisplayView& DisplayView) const
{
	if (DisplayView.IsValid())
	{
		return DesiredDisplayViews.Find(DisplayView) != INDEX_NONE;
	}

	return false;
}

bool UOpenColorIOConfiguration::Validate() const
{
#if WITH_OCIO
	if (!ConfigurationFile.FilePath.IsEmpty() && Config.IsValid())
	{
		//When loading the configuration file, if any errors are detected, it will throw an exception. Thus, our pointer won't be valid.
		return Config->IsValid();
	}

	return false;
#else
	return true;
#endif // WITH_OCIO
}

bool UOpenColorIOConfiguration::TransformColor(const FOpenColorIOColorConversionSettings& InSettings, FLinearColor& InOutColor) const
{
	TObjectPtr<const UOpenColorIOColorTransform> Transform = FindTransform(InSettings);
	if (IsValid(Transform))
	{
		return Transform->TransformColor(InOutColor);
	}

	return false;
}

bool UOpenColorIOConfiguration::TransformImage(const FOpenColorIOColorConversionSettings& InSettings, const FImageView& InOutImage) const
{
	TObjectPtr<const UOpenColorIOColorTransform> Transform = FindTransform(InSettings);
	if (IsValid(Transform))
	{
		return Transform->TransformImage(InOutImage);
	}

	return false;
}

bool UOpenColorIOConfiguration::TransformImage(const FOpenColorIOColorConversionSettings& InSettings, const FImageView& SrcImage, const FImageView& DestImage) const
{
	TObjectPtr<const UOpenColorIOColorTransform> Transform = FindTransform(InSettings);
	if (IsValid(Transform))
	{
		return Transform->TransformImage(SrcImage, DestImage);
	}

	return false;
}

void UOpenColorIOConfiguration::ReloadExistingColorspaces(bool bForce)
{
	LoadConfiguration();

#if WITH_OCIO
	if (Config && Config->IsValid())
	{
		FString NewConfigHash;
		if (!GetHash(NewConfigHash))
		{
			bForce = true;
		}

		// Hash is different, proceed with the regeneration...
		if (ConfigHash != NewConfigHash || bForce)
		{
			ConfigHash = NewConfigHash;

			TArray<FOpenColorIOColorSpace> ColorSpacesToBeReloaded = DesiredColorSpaces;
			TArray<FOpenColorIODisplayView> DisplayViewsToBeReloaded = DesiredDisplayViews;
			DesiredColorSpaces.Reset();
			DesiredDisplayViews.Reset();
			ColorTransforms.Reset();

			// This will make sure that all colorspaces are up to date in case an index, family or name is changed.
			for (const FOpenColorIOColorSpace& ExistingColorSpace : ColorSpacesToBeReloaded)
			{
				int32 ColorSpaceIndex = Config->GetColorSpaceIndex(*ExistingColorSpace.ColorSpaceName);
				if (ColorSpaceIndex < 0)
				{
					// Name not found, therefore we don't need to re-add this colorspace.
					UE_LOGF(LogOpenColorIO, Display, "Removing %ls, not found.", *ExistingColorSpace.ToString());
					continue;
				}

				FOpenColorIOColorSpace ColorSpace;
				ColorSpace.ColorSpaceName = ExistingColorSpace.ColorSpaceName;
				ColorSpace.FamilyName = Config->GetColorSpaceFamilyName(*ExistingColorSpace.ColorSpaceName);
				ColorSpace.Description = Config->GetColorSpaceDescription(*ExistingColorSpace.ColorSpaceName);

				DesiredColorSpaces.Add(ColorSpace);
			}

			for (const FOpenColorIODisplayView& ExistingDisplayView : DisplayViewsToBeReloaded)
			{
				bool bDisplayViewFound = false;
				for (int32 ViewIndex = 0; ViewIndex < Config->GetNumViews(*ExistingDisplayView.Display); ++ViewIndex)
				{
					const FString ViewName = Config->GetViewName(*ExistingDisplayView.Display, ViewIndex);

					if (ViewName == ExistingDisplayView.View)
					{
						FOpenColorIODisplayView DisplayView = ExistingDisplayView;
						DisplayView.Description = Config->GetDisplayViewTransformDescription(*ExistingDisplayView.Display, *ExistingDisplayView.View);
						
						DesiredDisplayViews.Add(DisplayView);
						bDisplayViewFound = true;
						break;
					}
				}

				if (!bDisplayViewFound)
				{
					UE_LOGF(LogOpenColorIO, Display, "Removing %ls, not found.", *ExistingDisplayView.ToString());
				}
			}

			const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();

			// Generate new shaders.
			for (int32 indexTop = 0; indexTop < DesiredColorSpaces.Num(); ++indexTop)
			{
				const FOpenColorIOColorSpace& TopColorSpace = DesiredColorSpaces[indexTop];

				for (int32 indexOther = indexTop + 1; indexOther < DesiredColorSpaces.Num(); ++indexOther)
				{
					const FOpenColorIOColorSpace& OtherColorSpace = DesiredColorSpaces[indexOther];

					CreateColorTransform(TopColorSpace.ColorSpaceName, OtherColorSpace.ColorSpaceName);
					CreateColorTransform(OtherColorSpace.ColorSpaceName, TopColorSpace.ColorSpaceName);
				}

				for (const FOpenColorIODisplayView& DisplayView : DesiredDisplayViews)
				{
					CreateColorTransform(TopColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Forward);

					if (Settings->bSupportInverseViewTransforms)
					{
						CreateColorTransform(TopColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Inverse);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogOpenColorIO, Verbose, "Reload skipped, no differences identified.");
		}
	}
	else
	{
		DesiredColorSpaces.Reset();
		DesiredDisplayViews.Reset();
		ColorTransforms.Reset();
		ConfigHash.Empty();
	}

	if (!MarkPackageDirty())
	{
		UE_LOGF(LogOpenColorIO, Display, "%ls should be resaved for faster loads. Changes in the config or the library version were detected.", *GetPathName());
	}
#endif
}

void UOpenColorIOConfiguration::ConfigPathChangedEvent(const TArray<FFileChangeData>& InFileChanges, const FString InFileMountPath)
{
#if WITH_EDITOR
	// We want to stop reacting to these events while the message is still up.
	if (WatchedDirectoryInfo.RawConfigChangedToast.IsValid())
	{
		return;
	}
	for (const FFileChangeData& FileChangeData : InFileChanges)
	{
		const FString FileExtension = FPaths::GetExtension(FileChangeData.Filename);
		if (FileExtension.IsEmpty() || !OCIODirectoryWatcher::OcioExtensions.Contains(FileExtension))
		{
			continue;
		}

		const FText DialogBody = FText::Format(LOCTEXT("OcioConfigChanged",
			"Files associated with OCIO config or luts have been modified externally. \
			\nWould you like to reload '{0}' configuration file?"),
			FText::FromString(GetName()));

		const FText ReloadRawConfigText = LOCTEXT("ReloadRawConfigConfirm", "Reload");
		const FText IgnoreReloadRawConfigText = LOCTEXT("IgnoreReloadRawConfig", "Ignore");


		FSimpleDelegate OnReloadRawConfig = FSimpleDelegate::CreateLambda([this]() { OnToastCallback(true); });
		FSimpleDelegate OnIgnoreReloadRawConfig = FSimpleDelegate::CreateLambda([this]() { OnToastCallback(false); });

		FNotificationInfo Info(DialogBody);
		Info.bFireAndForget = false;
		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.bUseSuccessFailIcons = false;
		Info.ButtonDetails.Add(FNotificationButtonInfo(ReloadRawConfigText, FText(), OnReloadRawConfig));
		Info.ButtonDetails.Add(FNotificationButtonInfo(IgnoreReloadRawConfigText, FText(), OnIgnoreReloadRawConfig));

		WatchedDirectoryInfo.RawConfigChangedToast = FSlateNotificationManager::Get().AddNotification(Info);

		if (WatchedDirectoryInfo.RawConfigChangedToast.IsValid())
		{
			WatchedDirectoryInfo.RawConfigChangedToast.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}

		break;
	}

#endif
}

FOpenColorIOWrapperConfig* UOpenColorIOConfiguration::GetConfigWrapper() const
{
#if WITH_OCIO
	return Config.Get();
#else
	return nullptr;
#endif
}

FOpenColorIOWrapperConfig* UOpenColorIOConfiguration::GetOrCreateConfigWrapper()
{
#if WITH_OCIO
	if (!Config.IsValid())
	{
		LoadConfiguration();
	}
	return Config.Get();
#else
	return nullptr;
#endif
}

void UOpenColorIOConfiguration::StartDirectoryWatch(const FString& FilePath)
{
#if WITH_EDITOR
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(OCIODirectoryWatcher::NAME_DirectoryWatcher);
	if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
	{
		FString FolderPath = FPaths::GetPath(FilePath);

		// Unregister watched folder since ocio config path has changed..
		StopDirectoryWatch();

		WatchedDirectoryInfo.FolderPath = MoveTemp(FolderPath);
		{
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				WatchedDirectoryInfo.FolderPath, IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UOpenColorIOConfiguration::ConfigPathChangedEvent, WatchedDirectoryInfo.FolderPath),
				WatchedDirectoryInfo.DirectoryWatcherHandle,
				/*Flags*/ 0
			);
		}
	}
#endif
}

void UOpenColorIOConfiguration::StopDirectoryWatch()
{
#if WITH_EDITOR
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(OCIODirectoryWatcher::NAME_DirectoryWatcher);
	if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
	{
		if (WatchedDirectoryInfo.DirectoryWatcherHandle.IsValid())
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(WatchedDirectoryInfo.FolderPath, WatchedDirectoryInfo.DirectoryWatcherHandle);
			WatchedDirectoryInfo.FolderPath.Empty();
		}
	}
#endif
}

TObjectPtr<const UOpenColorIOColorTransform> UOpenColorIOConfiguration::FindTransform(const FOpenColorIOColorConversionSettings& InSettings) const
{
	const TObjectPtr<UOpenColorIOColorTransform>* TransformPtr = ColorTransforms.FindByPredicate([&](const UOpenColorIOColorTransform* InTransform)
		{
			EOpenColorIOViewTransformDirection DisplayViewDirection;
			
			if (InTransform->GetDisplayViewDirection(DisplayViewDirection))
			{
				return InTransform->SourceColorSpace == InSettings.SourceColorSpace.ColorSpaceName && InTransform->Display == InSettings.DestinationDisplayView.Display && InTransform->View == InSettings.DestinationDisplayView.View && InSettings.DisplayViewDirection == DisplayViewDirection;
			}
			else
			{
				return InTransform->SourceColorSpace == InSettings.SourceColorSpace.ColorSpaceName && InTransform->DestinationColorSpace == InSettings.DestinationColorSpace.ColorSpaceName;
			}
		});

	return (TransformPtr != nullptr) ? *TransformPtr : nullptr;
}

void UOpenColorIOConfiguration::CreateColorTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace)
{
	if (InSourceColorSpace.IsEmpty() || InDestinationColorSpace.IsEmpty())
	{
		return;
	}

	if (HasTransform(InSourceColorSpace, InDestinationColorSpace))
	{
		UE_LOGF(LogOpenColorIO, Log, "OCIOConfig already contains %ls to %ls transform.", *InSourceColorSpace, *InDestinationColorSpace);
		return;
	}

	UOpenColorIOColorTransform* NewTransform = NewObject<UOpenColorIOColorTransform>(this, NAME_None, RF_NoFlags);
	const bool bSuccess = NewTransform->Initialize(InSourceColorSpace, InDestinationColorSpace);

	if (bSuccess)
	{
		ColorTransforms.Add(NewTransform);
	}
	else
	{
		UE_LOGF(LogOpenColorIO, Warning, "Could not create color space transform from %ls to %ls. Verify your OCIO config file, it may have errors in it.", *InSourceColorSpace, *InDestinationColorSpace);
	}
}

void UOpenColorIOConfiguration::CreateColorTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection)
{
	if (InSourceColorSpace.IsEmpty() || InDisplay.IsEmpty() || InView.IsEmpty())
	{
		return;
	}

	if (HasTransform(InSourceColorSpace, InDisplay, InView, InDirection))
	{
		UE_LOGF(LogOpenColorIO, Log, "OCIOConfig already contains %ls to %ls-%ls transform.", *InSourceColorSpace, *InDisplay, *InView);
		return;
	}

	UOpenColorIOColorTransform* NewTransform = NewObject<UOpenColorIOColorTransform>(this, NAME_None, RF_NoFlags);
	const bool bSuccess = NewTransform->Initialize(InSourceColorSpace, InDisplay, InView, InDirection);

	if (bSuccess)
	{
		ColorTransforms.Add(NewTransform);
	}
	else
	{
		UE_LOGF(LogOpenColorIO, Warning, "Could not create color space transform from %ls to %ls - %ls. Verify your OCIO config file, it may have errors in it.", *InSourceColorSpace, *InDisplay, *InView);
	}
}

void UOpenColorIOConfiguration::CleanupTransforms()
{
	ColorTransforms = ColorTransforms.FilterByPredicate([this](const TObjectPtr<UOpenColorIOColorTransform>& InTransform) -> bool
		{
			//Check if the source color space of this transform is desired. If not, remove that transform from the list.
			const FOpenColorIOColorSpace* FoundSourceColorPtr = DesiredColorSpaces.FindByPredicate([InTransform](const FOpenColorIOColorSpace& InOtherColorSpace)
				{
					return InOtherColorSpace.ColorSpaceName == InTransform->SourceColorSpace;
				});

			if (FoundSourceColorPtr == nullptr)
			{
				return false;
			}

			void* DestinationPtr = nullptr;

			//The source was there so check if the destination color space/display view of this transform is desired. If not, remove that transform from the list.
			if (InTransform->bIsDisplayViewType)
			{
				DestinationPtr = DesiredDisplayViews.FindByPredicate([InTransform](const FOpenColorIODisplayView& InOtherDisplayView)
					{
						return InOtherDisplayView.Display.Equals(InTransform->Display) && InOtherDisplayView.View.Equals(InTransform->View);
					});

			}
			else
			{
				DestinationPtr = DesiredColorSpaces.FindByPredicate([InTransform](const FOpenColorIOColorSpace& InOtherColorSpace)
					{
						return InOtherColorSpace.ColorSpaceName == InTransform->DestinationColorSpace;
					});
			}

			if (DestinationPtr == nullptr)
			{
				return false;
			}

			// Both source and destination are in the desired collections, so we keep the transform.
			return true;
		});
}


void UOpenColorIOConfiguration::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (!HasAnyFlags(RF_NeedLoad | RF_ClassDefaultObject))
	{
		ConfigurationFile.FilePath = TEXT("ocio://default");
		// Ensure the default built-in configuration is loaded.
		LoadConfiguration();
	}
#endif //WITH_EDITOR
}

void UOpenColorIOConfiguration::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void UOpenColorIOConfiguration::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OpenColorIOv251)
	{
		// Since ACES 2.0 transforms are significant breaking change, we ease the transition by keeping old configs at the previous version.
		// TODO: Refactor configuration assets so that the external config is no longer "live" and instead only used on import.
		if (ConfigurationFile.FilePath == TEXT("ocio://default") || ConfigurationFile.FilePath == TEXT("ocio://cg-config-latest"))
		{
			ConfigurationFile.FilePath = TEXT("ocio://cg-config-v2.2.0_aces-v1.3_ocio-v2.4");
		}
		else if (ConfigurationFile.FilePath == TEXT("ocio://studio-config-latest"))
		{
			ConfigurationFile.FilePath = TEXT("ocio://studio-config-v2.2.0_aces-v1.3_ocio-v2.4");
		}
	}

	// In non-editor modes, the shader & texture resources are already baked.
	// For CPU transforms, the config is lazily-loaded.
	ReloadExistingColorspaces();
#endif

	for (UOpenColorIOColorTransform* Transform : ColorTransforms)
	{
		Transform->ConditionalPostLoad();
	}
}

void UOpenColorIOConfiguration::GetAssetRegistryTags(FAssetRegistryTagsContext RegistryTagsContext) const
{
	Super::GetAssetRegistryTags(RegistryTagsContext);

	FString Description;
	if (ConfigurationFile.FilePath.IsEmpty())
	{
		Description = TEXT("No configuration selected.");
	}
	else if (!Validate())
	{
		Description = TEXT("Warning: Configuration is invalid. Verify the selected configuration file.");
	}
	else
	{
		Description = TEXT("Configuration: ") + ConfigurationFile.FilePath;
	}

	RegistryTagsContext.AddTag(FAssetRegistryTag(TEXT("ConfigurationFile"), Description, FAssetRegistryTag::TT_Hidden));
}

#if WITH_EDITORONLY_DATA
void UOpenColorIOConfiguration::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UOpenColorIOColorTransform::StaticClass()));
}
#endif

namespace OpenColorIOConfiguration
{
	static void SendAnalytics(const FString& EventName, const TArray<FOpenColorIOColorSpace>& DesiredColorSpaces)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("NumDesiredColorSpaces"), DesiredColorSpaces.Num()));

		FEngineAnalytics::GetProvider().RecordEvent(EventName, EventAttributes);
	}
}

void UOpenColorIOConfiguration::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	OpenColorIOConfiguration::SendAnalytics(TEXT("Usage.OpenColorIO.ConfigAssetSaved"), DesiredColorSpaces);
}

void UOpenColorIOConfiguration::LoadConfiguration()
{
#if WITH_OCIO
	Config.Reset();

	if (ConfigurationFile.FilePath.IsEmpty())
	{
		return;
	}

	bool bIsBuiltIn = false;
	FString FilePath = ConfigurationFile.FilePath;

	if (ConfigurationFile.FilePath.StartsWith(TEXT("ocio://")))
	{
		bIsBuiltIn = true;
	}
	else if (ConfigurationFile.FilePath.Equals(TEXT("$OCIO")) || ConfigurationFile.FilePath.Equals(TEXT("%OCIO%")))
	{
		FilePath = FPlatformMisc::GetEnvironmentVariable(TEXT("OCIO"));
	}
	else if (ConfigurationFile.FilePath.Contains(TEXT("{Engine}")))
	{
		FilePath = FPaths::ConvertRelativePathToFull(ConfigurationFile.FilePath.Replace(TEXT("{Engine}"), *FPaths::EngineDir()));
	}
	else if (FPaths::IsRelative(ConfigurationFile.FilePath))
	{
		const FString AbsoluteGameDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
		FilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(AbsoluteGameDir, ConfigurationFile.FilePath));
	}

	Config = MakePimpl<FOpenColorIOWrapperConfig>(FilePath, FOpenColorIOWrapperConfig::WCS_AsInterchangeSpace);

	if (Config->IsValid())
	{
		if (!bIsBuiltIn)
		{
			StartDirectoryWatch(FilePath);
		}

		UE_LOGF(LogOpenColorIO, Verbose, "Loaded OCIO configuration file %ls", *FilePath);
	}
	else
	{
		UE_LOGF(LogOpenColorIO, Error, "Could not load OCIO configuration file %ls. Verify that the path is good or that the file is valid.", *FilePath);
	}
#endif //WITH_OCIO
}

bool UOpenColorIOConfiguration::GetHash(FString& OutHash) const
{
#if WITH_OCIO
	if (Config && Config->IsValid())
	{
		const FString CacheID = Config->GetCacheID();
		if (CacheID.IsEmpty())
		{
			UE_LOGF(LogOpenColorIO, Warning, "Failed to get cache ID: forcing constant transform recreation. Please fix invalid config.");
			return false;
		}

		const FStringView CacheIDView = CacheID;
		const uint32 VersionID = OpenColorIOWrapper::GetVersionHex();
		const uint8 bSupportsInverse = GetDefault<UOpenColorIOSettings>()->bSupportInverseViewTransforms;

		// Note that we use 128-bit xxhash to match the library.
		// See: https://github.com/AcademySoftwareFoundation/OpenColorIO/wiki/Caches-and-cache-IDs-in-OpenColorIO#object-cacheids

		FXxHash128Builder HashBuilder;
		HashBuilder.Update(CacheIDView.GetData(), CacheIDView.Len() * sizeof(CacheIDView[0]));
		HashBuilder.Update(&VersionID, sizeof(VersionID));
		HashBuilder.Update(&bSupportsInverse, sizeof(bSupportsInverse));

		const UE::Color::FColorSpace& WCS = UE::Color::FColorSpace::GetWorking();
		if (!WCS.IsSRGB())
		{
			// The working color space is uniquely defined by its chromaticities (as loaded from renderer settings).
			HashBuilder.Update(&WCS.GetRedChromaticity(), sizeof(FVector2d));
			HashBuilder.Update(&WCS.GetGreenChromaticity(), sizeof(FVector2d));
			HashBuilder.Update(&WCS.GetBlueChromaticity(), sizeof(FVector2d));
			HashBuilder.Update(&WCS.GetWhiteChromaticity(), sizeof(FVector2d));
		}

		for (const TPair<FString, FString>& Pair : Context)
		{
			if (!Pair.Value.IsEmpty())
			{
				const FStringView Key = Pair.Key;
				const FStringView Value = Pair.Value;

				HashBuilder.Update(Key.GetData(), Key.Len() * sizeof(Key[0]));
				HashBuilder.Update(Value.GetData(), Value.Len() * sizeof(Value[0]));
			}
		}
		
		TStringBuilder<32> StringBuilder;
		StringBuilder << HashBuilder.Finalize();
		OutHash = FString(StringBuilder);

		return true;
	}
#endif // WITH_OCIO

	return false;
}

#if WITH_EDITOR

void UOpenColorIOConfiguration::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, ConfigurationFile))
	{
		// Note: Reload calls LoadConfiguration() internally.
		SetConfigurationFile(ConfigurationFile);
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, DesiredColorSpaces))
	{
		if (PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayAdd | EPropertyChangeType::Duplicate | EPropertyChangeType::ValueSet))
		{
			for (int32 indexTop = 0; indexTop < DesiredColorSpaces.Num(); ++indexTop)
			{
				const FOpenColorIOColorSpace& TopColorSpace = DesiredColorSpaces[indexTop];

				for (int32 indexOther = indexTop + 1; indexOther < DesiredColorSpaces.Num(); ++indexOther)
				{
					const FOpenColorIOColorSpace& OtherColorSpace = DesiredColorSpaces[indexOther];

					CreateColorTransform(TopColorSpace.ColorSpaceName, OtherColorSpace.ColorSpaceName);
					CreateColorTransform(OtherColorSpace.ColorSpaceName, TopColorSpace.ColorSpaceName);
				}

				for (const FOpenColorIODisplayView& DisplayView : DesiredDisplayViews)
				{
					CreateColorTransform(TopColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Forward);

					if (Settings->bSupportInverseViewTransforms)
					{
						CreateColorTransform(TopColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Inverse);
					}
				}
			}
		}

		if (PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayRemove | EPropertyChangeType::ArrayClear | EPropertyChangeType::ValueSet))
		{
			CleanupTransforms();
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, DesiredDisplayViews))
	{
		if (PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayAdd | EPropertyChangeType::Duplicate | EPropertyChangeType::ValueSet))
		{
			for (const FOpenColorIODisplayView& DisplayView : DesiredDisplayViews)
			{
				for (const FOpenColorIOColorSpace& SourceColorSpace : DesiredColorSpaces)
				{
					CreateColorTransform(SourceColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Forward);

					if (Settings->bSupportInverseViewTransforms)
					{
						CreateColorTransform(SourceColorSpace.ColorSpaceName, DisplayView.Display, DisplayView.View, EOpenColorIOViewTransformDirection::Inverse);
					}
				}
			}
		}

		if (PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayRemove | EPropertyChangeType::ArrayClear | EPropertyChangeType::ValueSet))
		{
			CleanupTransforms();
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, Context))
	{
		// Note: Reload calls LoadConfiguration() internally.
		ReloadExistingColorspaces();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UOpenColorIOConfiguration::OnToastCallback(bool bInReloadColorspaces)
{
	if (WatchedDirectoryInfo.RawConfigChangedToast.IsValid())
	{
		WatchedDirectoryInfo.RawConfigChangedToast.Pin()->SetCompletionState(SNotificationItem::CS_Success);
		WatchedDirectoryInfo.RawConfigChangedToast.Pin()->ExpireAndFadeout();
		WatchedDirectoryInfo.RawConfigChangedToast.Reset();
	}

	if (bInReloadColorspaces)
	{
		OpenColorIOWrapper::ClearAllCaches();

		ReloadExistingColorspaces();
	}
}

#endif //WITH_EDITOR

void UOpenColorIOConfiguration::SetConfigurationFile(const FFilePath& InFilePath)
{
	ConfigurationFile = InFilePath;

	const FString EngineFullPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());

	if (!EngineFullPath.IsEmpty() && ConfigurationFile.FilePath.Contains(EngineFullPath))
	{
		// Automatically replace the engine path with its {Engine} token for better compatilibilty across different machines.
		ConfigurationFile.FilePath = ConfigurationFile.FilePath.Replace(*EngineFullPath, TEXT("{Engine}/"));
	}

	ReloadExistingColorspaces();
}

#undef LOCTEXT_NAMESPACE
