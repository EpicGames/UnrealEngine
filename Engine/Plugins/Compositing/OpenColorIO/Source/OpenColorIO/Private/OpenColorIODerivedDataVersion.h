// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
OpenColorIOShaderDerivedDataVersion.h: Derived data version for OpenColorIO shaders and LUTs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new
// guid as version
#define OPENCOLORIO_DERIVEDDATA_VER		TEXT("EF2734CC-A92C-4025-A75B-96AB950E5198")
