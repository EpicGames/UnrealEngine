// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIODisplayExtension.h"

#include "Containers/SortedMap.h"
#include "CoreGlobals.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOColorTransform.h"
#include "OpenColorIODisplayManager.h"
#include "OpenColorIOModule.h"

#include "OpenColorIOShader.h"
#include "Shader.h"

#include "RHI.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "CommonRenderResources.h"
#include "Containers/DynamicRHIResourceArray.h"
// for FPostProcessMaterialInputs
#include "PostProcess/PostProcessMaterialInputs.h"


FOpenColorIODisplayExtension::FOpenColorIODisplayExtension(const FAutoRegister& AutoRegister, FViewportClient* AssociatedViewportClient)
	: FSceneViewExtensionBase(AutoRegister)
	, LinkedViewportClient(AssociatedViewportClient)
{
}

bool FOpenColorIODisplayExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	if (Context.Viewport && LinkedViewportClient == Context.Viewport->GetClient())
	{
		return DisplayConfiguration.bIsEnabled;
	}

	return false;
}

void FOpenColorIODisplayExtension::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (DisplayConfiguration.ColorConfiguration.ConfigurationSource)
	{
		Collector.AddReferencedObject(DisplayConfiguration.ColorConfiguration.ConfigurationSource);
	}
}

void FOpenColorIODisplayExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	FOpenColorIORenderPassResources PassResources = FOpenColorIORendering::GetRenderPassResources(DisplayConfiguration.ColorConfiguration, InViewFamily.GetFeatureLevel());

	if (PassResources.IsValid())
	{
		FOpenColorIORendering::PrepareView(InViewFamily, InView);
	}
	ENQUEUE_RENDER_COMMAND(ProcessColorSpaceTransform)(
		[WeakThis = AsWeak(), ResourcesRenderThread = MoveTemp(PassResources)](FRHICommandListImmediate& RHICmdList)
		{
			TSharedPtr<FOpenColorIODisplayExtension> This = StaticCastWeakPtr<FOpenColorIODisplayExtension>(WeakThis).Pin();
			if (This.IsValid())
			{
				//Caches render thread resource to be used when applying configuration in PostRenderViewFamily_RenderThread
				This->CachedResourcesRenderThread = ResourcesRenderThread;
			}
		}
	);
}

void FOpenColorIODisplayExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (PassId == EPostProcessingPass::Tonemap)
	{
		InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FOpenColorIODisplayExtension::PostProcessPassAfterTonemap_RenderThread));
	}
}

FScreenPassTexture FOpenColorIODisplayExtension::PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
{
	const FScreenPassTexture& SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));
	check(SceneColor.IsValid());

	FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

	// If the override output is provided, it means that this is the last pass in post processing.
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, SceneColor, View.GetOverwriteLoadAction(), TEXT("OCIORenderTarget"));
	}

	FOpenColorIORendering::AddPass_RenderThread(
		GraphBuilder,
		View,
		SceneColor,
		Output,
		CachedResourcesRenderThread
	);

	return MoveTemp(Output);
}