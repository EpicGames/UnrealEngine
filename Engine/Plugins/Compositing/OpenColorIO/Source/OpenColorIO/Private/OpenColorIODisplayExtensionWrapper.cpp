// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIODisplayExtensionWrapper.h"
#include "OpenColorIOModule.h"
#include "Engine/GameEngine.h"
#include "Slate/SceneViewport.h"

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIODisplayExtensionWrapper)

void UOpenColorIODisplayExtensionWrapper::CreateDisplayExtensionIfNotExists()
{
	if (!DisplayExtension.IsValid())
	{
		// Null viewport should ensure it doesn't run anywhere yet (unless explicitly gathered)
		DisplayExtension = FSceneViewExtensions::NewExtension<FOpenColorIODisplayExtension>(nullptr);
	}
	check(DisplayExtension.IsValid());
}

FOpenColorIODisplayConfiguration UOpenColorIODisplayExtensionWrapper::GetOpenColorIOConfiguration() const
{
	if (!DisplayExtension.IsValid())
	{
		return {};
	}

	return DisplayExtension->GetDisplayConfiguration();
}

void UOpenColorIODisplayExtensionWrapper::SetOpenColorIOConfiguration(FOpenColorIODisplayConfiguration InDisplayConfiguration)
{
	if (!DisplayExtension.IsValid())
	{
		return;
	}

	DisplayExtension->SetDisplayConfiguration(InDisplayConfiguration);
}

void UOpenColorIODisplayExtensionWrapper::SetSceneExtensionIsActiveFunction(const FSceneViewExtensionIsActiveFunctor& IsActiveFunction)
{
	if (!DisplayExtension.IsValid())
	{
		return;
	}

	DisplayExtension->IsActiveThisFrameFunctions.Reset(1);
	DisplayExtension->IsActiveThisFrameFunctions.Add(IsActiveFunction);
}

void UOpenColorIODisplayExtensionWrapper::SetSceneExtensionIsActiveFunctions(const TArray<FSceneViewExtensionIsActiveFunctor>& IsActiveFunctions)
{
	if (!DisplayExtension.IsValid())
	{
		return;
	}

	DisplayExtension->IsActiveThisFrameFunctions = IsActiveFunctions;
}

void UOpenColorIODisplayExtensionWrapper::RemoveSceneExtension()
{
	// We disable the display configuration before deletion so it's not accidentally kept alive by the GatherActiveExtensions() shared pointers.
	if (DisplayExtension.IsValid())
	{
		DisplayExtension->GetDisplayConfiguration().bIsEnabled = false;
	}

	DisplayExtension.Reset();
}

UOpenColorIODisplayExtensionWrapper* UOpenColorIODisplayExtensionWrapper::CreateOpenColorIODisplayExtension(
	FOpenColorIODisplayConfiguration InDisplayConfiguration,
	const FSceneViewExtensionIsActiveFunctor& IsActiveFunction)
{
	// Create OCIO Scene View Extension and configure it.

	UOpenColorIODisplayExtensionWrapper* OutExtension = NewObject<UOpenColorIODisplayExtensionWrapper>();

	OutExtension->CreateDisplayExtensionIfNotExists();
	OutExtension->SetOpenColorIOConfiguration(InDisplayConfiguration);
	OutExtension->SetSceneExtensionIsActiveFunction(IsActiveFunction);

	return OutExtension;
}

UOpenColorIODisplayExtensionWrapper* UOpenColorIODisplayExtensionWrapper::CreateInGameOpenColorIODisplayExtension(FOpenColorIODisplayConfiguration InDisplayConfiguration)
{
	FSceneViewExtensionIsActiveFunctor IsActiveFunctor;
	IsActiveFunctor.IsActiveFunction = [](const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context)
	{
		// Note: SceneViewExtension this pointer is always passed to functors from FSceneViewExtensionBase::IsActiveThisFrame, we confirm this remains so.
		check(SceneViewExtension);

		const FOpenColorIODisplayExtension* SVE = static_cast<const FOpenColorIODisplayExtension*>(SceneViewExtension);
		const bool bIsEnabled = SVE->GetDisplayConfiguration().bIsEnabled;

		if (!Context.Viewport)
		{
			return TOptional<bool>();
		}

#if WITH_EDITOR
		if (GIsEditor && GEditor)
		{
			// Activate the SVE if it is a PIE viewport.
			if (Context.Viewport->IsPlayInEditorViewport())
			{
				return TOptional<bool>(bIsEnabled);
			}
		}
#endif // WITH_EDITOR

		// Activate the SVE if it is the game's primary viewport.
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			if (GameEngine->SceneViewport.IsValid() && (GameEngine->SceneViewport.Get() == Context.Viewport))
			{
				return TOptional<bool>(bIsEnabled);
			}
		}

		// Activate the SVE if a high-res screenshot is being taken
		if (GIsHighResScreenshot)
		{
			return TOptional<bool>(bIsEnabled);
		}

		// If our viewport did not meet any of the criteria to activate the SVE, emit no opinion.
		return TOptional<bool>();
	};

	return CreateOpenColorIODisplayExtension(InDisplayConfiguration, IsActiveFunctor);
}
