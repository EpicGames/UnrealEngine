// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIODisplayManager.h"

#include "OpenColorIODisplayExtension.h"

FOpenColorIODisplayConfiguration& FOpenColorIODisplayManager::FindOrAddDisplayConfiguration(FViewportClient* InViewportClient)
{
	const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>* Extension = DisplayExtensions.FindByPredicate([InViewportClient](const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>& Other) { return  Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient(); });
	if (Extension)
	{
		FOpenColorIODisplayExtension* ExtensionPtr = Extension->Get();
		check(ExtensionPtr);

		FOpenColorIODisplayConfiguration& DisplayConfiguration = ExtensionPtr->GetDisplayConfiguration();
		DisplayConfiguration.ColorConfiguration.ValidateColorSpaces();
		return DisplayConfiguration;
	}
	
	// Extension not found, create it and return its config
	TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe> DisplayExtension = FSceneViewExtensions::NewExtension<FOpenColorIODisplayExtension>(InViewportClient);
	DisplayExtensions.Add(DisplayExtension);
	return DisplayExtension->GetDisplayConfiguration();
}

const FOpenColorIODisplayConfiguration* FOpenColorIODisplayManager::GetDisplayConfiguration(const FViewportClient* InViewportClient) const
{
	const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>* Extension = DisplayExtensions.FindByPredicate([InViewportClient](const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>& Other) { return  Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient(); });
	if (Extension)
	{
		FOpenColorIODisplayExtension* ExtensionPtr = Extension->Get();
		check(ExtensionPtr);

		FOpenColorIODisplayConfiguration& DisplayConfiguration = ExtensionPtr->GetDisplayConfiguration();
		DisplayConfiguration.ColorConfiguration.ValidateColorSpaces();
		return &DisplayConfiguration;
	}

	return nullptr;
}

bool FOpenColorIODisplayManager::RemoveDisplayConfiguration(const FViewportClient* InViewportClient)
{
	const int32 Index = DisplayExtensions.IndexOfByPredicate([InViewportClient](const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>& Other) { return  Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient(); });
	if (Index != INDEX_NONE)
	{
		DisplayExtensions.RemoveAtSwap(Index);
		return true;
	}

	return false;
}

bool FOpenColorIODisplayManager::IsTrackingViewport(const FViewportClient* InViewportClient) const
{
	return DisplayExtensions.ContainsByPredicate([InViewportClient](const TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>& Other) { return  Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient(); });
}

