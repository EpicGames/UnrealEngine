// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOModule.h"

#include "Interfaces/IPluginManager.h"
#include "OpenColorIODisplayManager.h"
#include "ShaderCore.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "OpenColorIOWrapperModule.h"
#endif

DEFINE_LOG_CATEGORY(LogOpenColorIO);

#define LOCTEXT_NAMESPACE "OpenColorIOModule"

FOpenColorIOModule::FOpenColorIOModule()
	: DisplayManager(MakeUnique<FOpenColorIODisplayManager>())
{

}

void FOpenColorIOModule::StartupModule()
{
#if WITH_EDITOR
	// Ensure the wrapper/library modules are loaded first.
	IOpenColorIOWrapperModule::Get();
#endif

	// Maps virtual shader source directory /Plugin/OpenCVLensDistortion to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("OpenColorIO"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/OpenColorIO"), PluginShaderDir);
}

void FOpenColorIOModule::ShutdownModule()
{

}

FOpenColorIODisplayManager& FOpenColorIOModule::GetDisplayManager()
{
	return *DisplayManager;
}
	
IMPLEMENT_MODULE(FOpenColorIOModule, OpenColorIO);

#undef LOCTEXT_NAMESPACE
