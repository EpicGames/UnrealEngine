// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenColorIOModule.h"

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Templates/UniquePtr.h"


DECLARE_LOG_CATEGORY_EXTERN(LogOpenColorIO, Log, All);


class FOpenColorIODisplayManager;

/**
 * Implements the OpenColorIO module.
 */
class FOpenColorIOModule : public IOpenColorIOModule
{
public:

	FOpenColorIOModule();
	virtual ~FOpenColorIOModule() = default;

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ Begin IOpenColorIOModule
	virtual FOpenColorIODisplayManager& GetDisplayManager() override;
	//~ End IOpenColorIOModule

private:

	TUniquePtr<FOpenColorIODisplayManager> DisplayManager;
};

