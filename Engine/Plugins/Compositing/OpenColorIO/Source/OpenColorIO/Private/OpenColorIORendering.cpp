// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIORendering.h"

#include "Engine/RendererSettings.h"
#include "Engine/TextureRenderTarget.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "GlobalShader.h"
#include "HDRHelper.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOModule.h"
#include "OpenColorIOShader.h"
#include "OpenColorIOShaderType.h"
#include "OpenColorIOShared.h"
#include "OpenColorIOColorTransform.h"
#include "SceneInterface.h"
#include "ScreenPass.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"

// static
void FOpenColorIORendering::AddPass_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FScreenPassViewInfo ViewInfo,
	ERHIFeatureLevel::Type FeatureLevel,
	const FScreenPassTexture& Input,
	const FScreenPassRenderTarget& Output,
	const FOpenColorIORenderPassResources& InPassResource,
	float InGamma, 
	EOpenColorIOTransformAlpha TransformAlpha)
{
	check(IsInRenderingThread());

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);

	TArray<FTextureResource*> TextureResources;
	TextureResources.Reserve(InPassResource.TextureResources.Num());

	for (const TWeakObjectPtr<UTexture>& Texture : InPassResource.TextureResources)
	{
		if (Texture.IsValid(false, true))
		{
			// UTexture's GetResource() is safe to call from the render thread.
			if (FTextureResource* TextureResource = Texture->GetResource())
			{
				if (TextureResource->TextureRHI.IsValid())
				{
					TextureResources.Add(TextureResource);
				}
			}
		}
	}

	const bool bAllExpectedTexturesResources = TextureResources.Num() == InPassResource.TextureResources.Num();
	const TSharedPtr<const FShaderParametersMetadata, ESPMode::ThreadSafe> ParametersMetadata = InPassResource.ShaderResource ? InPassResource.ShaderResource->GetShaderParamMetadata() : nullptr;

	if (bAllExpectedTexturesResources && InPassResource.ShaderResource != nullptr && ParametersMetadata.IsValid())
	{
		TShaderRef<FOpenColorIOPixelShader> PixelShader = InPassResource.ShaderResource->GetShader<FOpenColorIOPixelShader>();
		FOpenColorIOPixelShader::FParameters* PixelShaderParameters = GraphBuilder.AllocParameters<FOpenColorIOPixelShader::FParameters>(ParametersMetadata.Get());
		
		uint8* ParameterBufferPtr = reinterpret_cast<uint8*>(PixelShaderParameters);
		const TArray<FShaderParametersMetadata::FMember>& ParamMembers = ParametersMetadata->GetMembers();

		FOpenColorIOPixelShaderParameters* Parameters = reinterpret_cast<FOpenColorIOPixelShaderParameters*>(ParameterBufferPtr);
		Parameters->InputTexture = Input.Texture;
		Parameters->InputTextureSampler = TStaticSamplerState<>::GetRHI();
		Parameters->Gamma = InGamma;
		Parameters->TransformAlpha = static_cast<uint32>(TransformAlpha);
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		
		// Dynamic texture parameters (generated by OpenColorIO) are appended after our built-in struct of parameters.
		const int32 StartMemberTextureIndex = 1;
		for (int32 MemberIndex = StartMemberTextureIndex; MemberIndex < ParamMembers.Num(); ++MemberIndex)
		{
			const int32 TextureIndex = (MemberIndex - StartMemberTextureIndex) / 2;

			const FShaderParametersMetadata::FMember& Member = ParamMembers[MemberIndex];
			switch (Member.GetBaseType())
			{
			case UBMT_TEXTURE:
				*((FRHITexture**)(ParameterBufferPtr + Member.GetOffset())) = TextureResources[TextureIndex]->TextureRHI.GetReference();
				break;
			case UBMT_SAMPLER:
				*((FRHISamplerState**)(ParameterBufferPtr + Member.GetOffset())) = TextureResources[TextureIndex]->SamplerStateRHI.GetReference();
				break;

			default:
				checkNoEntry();
				break;
			}
		}

		check(VertexShader.IsValid());
		check(PixelShader.IsValid());
		check(PixelShaderParameters);

		ClearUnusedGraphResources(PixelShader, ParametersMetadata.Get(), PixelShaderParameters);

		FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
		const FScreenPassPipelineState PipelineState(VertexShader, PixelShader, BlendState, DepthStencilState);

		FRDGEventName PassName = RDG_EVENT_NAME("OpenColorIOPass %dx%d (%s)", Output.ViewRect.Width(), Output.ViewRect.Height(), InPassResource.TransformName.IsEmpty() ? TEXT("Unspecified Transform") : *InPassResource.TransformName);
		GraphBuilder.AddPass(
			Forward<FRDGEventName&&>(PassName),
			ParametersMetadata.Get(),
			PixelShaderParameters,
			ERDGPassFlags::Raster,
			[ViewInfo, OutputViewport, InputViewport, PipelineState, ParametersMetadata, PixelShaderParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				constexpr EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None;
				DrawScreenPass(RHICmdList, ViewInfo, OutputViewport, InputViewport, PipelineState, Flags, [&](FRHICommandList&)
					{
						SetShaderParameters(RHICmdList, PipelineState.PixelShader.GetPixelShader(), PipelineState.PixelShader->Bindings, ParametersMetadata.Get(), PixelShaderParameters);
					});
			});
	}
	else
	{
		// Fallback pass, printing invalid message across the viewport.
		TShaderMapRef<FOpenColorIOInvalidPixelShader> OCIOInvalidPixelShader(ShaderMap);
		FOpenColorIOInvalidShaderParameters* Parameters = GraphBuilder.AllocParameters<FOpenColorIOInvalidShaderParameters>();
		Parameters->InputTexture = Input.Texture;
		Parameters->InputTextureSampler = TStaticSamplerState<>::GetRHI();
		Parameters->MiniFontTexture = OpenColorIOGetMiniFontTexture();
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		FRDGEventName PassName = RDG_EVENT_NAME("OpenColorIOInvalidPass %dx%d", Output.ViewRect.Width(), Output.ViewRect.Height());
		AddDrawScreenPass(GraphBuilder, MoveTemp(PassName), ViewInfo, OutputViewport, InputViewport, VertexShader, OCIOInvalidPixelShader, Parameters);
	}
}

// static
void FOpenColorIORendering::AddPass_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FScreenPassTexture& Input,
	const FScreenPassRenderTarget& Output,
	const FOpenColorIORenderPassResources& InPassInfo,
	EOpenColorIOTransformAlpha TransformAlpha) 
{
	const float EngineDisplayGamma = View.Family->RenderTarget->GetDisplayGamma();
	const float DefaultDisplayGammaRT = UTextureRenderTarget::GetDefaultDisplayGamma();
	float DisplayGamma = DefaultDisplayGammaRT / EngineDisplayGamma;

	if (IsDefaultBackBufferLinearSDR())
	{
		// Note: To match editor in this mode, transforms need to output linear colors.
		DisplayGamma = 1.0f;
	}
	// There is a special case where post processing and tonemapper are disabled. In this case tonemapper applies a static display Inverse of Gamma which defaults to 2.2.
	else if ((View.Family->EngineShowFlags.Tonemapper == 0) || (View.Family->EngineShowFlags.PostProcessing == 0))
	{
		DisplayGamma = DefaultDisplayGammaRT;
	}

	FOpenColorIORendering::AddPass_RenderThread(
		GraphBuilder,
		View,
		View.GetFeatureLevel(),
		Input,
		Output,
		InPassInfo,
		DisplayGamma,
		TransformAlpha
	);
}

void FOpenColorIORendering::PrepareView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	//Force ToneCurve to be off while we'are alive to make sure the input color space is the working space : srgb linear
	InViewFamily.EngineShowFlags.SetToneCurve(false);
	// This flags sets tonampper to output to ETonemapperOutputDevice::LinearNoToneCurve
	InViewFamily.SceneCaptureSource = SCS_FinalColorHDR;

	InView.FinalPostProcessSettings.bOverride_ToneCurveAmount = 1;
	InView.FinalPostProcessSettings.ToneCurveAmount = 0.0;
}

// static
FOpenColorIORenderPassResources FOpenColorIORendering::GetRenderPassResources(const FOpenColorIOColorConversionSettings& InSettings, ERHIFeatureLevel::Type InFeatureLevel)
{
	FOpenColorIORenderPassResources Result = { nullptr, {}, InSettings.ToString() };

	if (InSettings.ConfigurationSource != nullptr)
	{
		const bool bFoundTransform = InSettings.ConfigurationSource->GetRenderResources(
			InFeatureLevel,
			InSettings,
			Result.ShaderResource,
			Result.TextureResources
		);

		if (bFoundTransform)
		{
			// Transform was found, so shader must be there but doesn't mean the actual shader is available
			check(Result.ShaderResource);
			if (Result.ShaderResource->GetShaderGameThread<FOpenColorIOPixelShader>().IsNull())
			{
				//Invalidate shader resource
				Result.ShaderResource = nullptr;
			}
		}
	}

	return Result;
}

// static
bool FOpenColorIORendering::ApplyColorTransform(UWorld* InWorld, const FOpenColorIOColorConversionSettings& InSettings, UTexture* InTexture, UTextureRenderTarget2D* OutRenderTarget)
{
	check(IsInGameThread());

	if (!ensureMsgf(InTexture, TEXT("Can't apply color transform - Invalid Input Texture")))
	{
		return false;
	}

	if (!ensureMsgf(OutRenderTarget, TEXT("Can't apply color transform - Invalid Output Texture")))
	{
		return false;
	}

	FTextureResource* InputResource = InTexture->GetResource();
	FTextureResource* OutputResource = OutRenderTarget->GetResource();
	if (!ensureMsgf(InputResource, TEXT("Can't apply color transform - Invalid Input Texture resource")))
	{
		return false;
	}

	if (!ensureMsgf(OutputResource, TEXT("Can't apply color transform - Invalid Output Texture resource")))
	{
		return false;
	}

	const ERHIFeatureLevel::Type FeatureLevel = InWorld->Scene->GetFeatureLevel();
	FOpenColorIOTransformResource* ShaderResource = nullptr;
	TArray<TWeakObjectPtr<UTexture>> TransformTextures;

	if (InSettings.ConfigurationSource != nullptr)
	{
		const bool bFoundTransform = InSettings.ConfigurationSource->GetRenderResources(FeatureLevel, InSettings, ShaderResource, TransformTextures);

		if (bFoundTransform)
		{
			check(ShaderResource);
			if (ShaderResource->GetShaderGameThread<FOpenColorIOPixelShader>().IsNull())
			{
				ensureMsgf(false, TEXT("Can't apply display look - Shader was invalid for Resource %s"), *ShaderResource->GetFriendlyName());

				//Invalidate shader resource
				ShaderResource = nullptr;
			}
		}
	}
	
	ENQUEUE_RENDER_COMMAND(ProcessColorSpaceTransform)(
		[FeatureLevel, InputResource, OutputResource, ShaderResource, Textures = MoveTemp(TransformTextures), TransformName = InSettings.ToString()](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InputResource->TextureRHI, TEXT("OCIOInputTexture")));
			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OCIORenderTargetTexture")));
			FIntPoint  OutputResolution = FIntPoint(OutputResource->GetSizeX(), OutputResource->GetSizeY());
			FScreenPassRenderTarget Output = FScreenPassRenderTarget(OutputTexture, FIntRect(FIntPoint::ZeroValue, OutputResolution), ERenderTargetLoadAction::EClear);

			AddPass_RenderThread(
				GraphBuilder,
				FScreenPassViewInfo(),
				FeatureLevel,
				FScreenPassTexture(InputTexture),
				Output,
				FOpenColorIORenderPassResources{ShaderResource, Textures, TransformName},
				1.0f); // Set Gamma to 1., since we do not have any display parameters or requirement for Gamma.

			GraphBuilder.Execute();
		}
	);

	return ShaderResource != nullptr;
}
