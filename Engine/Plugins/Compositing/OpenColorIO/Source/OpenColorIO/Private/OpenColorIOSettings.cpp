// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOSettings)

UOpenColorIOSettings::UOpenColorIOSettings()
	: bUseLegacyProcessor(false)
	, bUse32fLUT(false)
	, bSupportInverseViewTransforms(false)
{

}

FName UOpenColorIOSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UOpenColorIOSettings::GetSectionText() const
{
	return NSLOCTEXT("OpenColorIOSettings", "OpenColorIOSettingsSection", "OpenColorIO");
}

FName UOpenColorIOSettings::GetSectionName() const
{
	return TEXT("OpenColorIO");
}
#endif
