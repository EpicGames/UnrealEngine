// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOShader.h"

#include "OpenColorIOModule.h"
#include "SystemTextures.h"

FOpenColorIOPixelShader::FOpenColorIOPixelShader(const FOpenColorIOShaderType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	constexpr bool bShouldBindEverything = true;
	const TSharedRef<const FShaderParametersMetadata>& ShaderParametersMetadata =
		static_cast<const FOpenColorIOShaderType::FParameters*>(Initializer.Parameters)->ShaderParamMetadata;

	Bindings.BindForLegacyShaderParameters(
		this,
		Initializer.PermutationId,
		Initializer.ParameterMap,
		ShaderParametersMetadata.Get(),
		bShouldBindEverything
	);
}

FRHITexture* OpenColorIOGetMiniFontTexture()
{
	return GSystemTextures.AsciiTexture ? GSystemTextures.AsciiTexture->GetRHI() : GSystemTextures.WhiteDummy->GetRHI();
}

IMPLEMENT_SHADER_TYPE(, FOpenColorIOPixelShader, TEXT("/Plugin/OpenColorIO/Private/OpenColorIOShader.usf"), TEXT("MainPS"), SF_Pixel)
IMPLEMENT_SHADER_TYPE(, FOpenColorIOInvalidPixelShader, TEXT("/Plugin/OpenColorIO/Private/OpenColorIOInvalidShader.usf"), TEXT("MainPS"), SF_Pixel);
