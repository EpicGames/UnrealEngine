// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOShaderCompilationManager.h"

#if WITH_EDITOR

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "OpenColorIOShared.h"
#include "ShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"

#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatformManagerModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogOpenColorIOShaderCompiler, All, All);

static int32 GShowOpenColorIOShaderWarnings = 1;
static FAutoConsoleVariableRef CVarShowOpenColorIOShaderWarnings(
	TEXT("OpenColorIO.ShowShaderCompilerWarnings"),
	GShowOpenColorIOShaderWarnings,
	TEXT("When set to 1, will display all warnings from OpenColorIO shader compiles.")
	);

FOpenColorIOShaderCompilationManager GOpenColorIOShaderCompilationManager;

void FOpenColorIOShaderCompilationManager::Tick(float DeltaSeconds)
{
	ProcessAsyncResults();
}

FOpenColorIOShaderCompilationManager::FOpenColorIOShaderCompilationManager()
{
	
}

FOpenColorIOShaderCompilationManager::~FOpenColorIOShaderCompilationManager()
{
	for (FOpenColorIOShaderCompileWorkerInfo* Info : WorkerInfos)
	{
		delete Info;
	}

	WorkerInfos.Empty();
}

void FOpenColorIOShaderCompilationManager::InitWorkerInfo()
{
	if (WorkerInfos.Num() == 0) // Check to see if it has been initialized or not
	{
		// Ew. Should we just use FShaderCompilingManager's workers instead? Is that safe?
		const int32 NumVirtualCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
		const uint32 NumOpenColorIOShaderCompilingThreads = FMath::Min(NumVirtualCores - 1, 4);

		for (uint32 WorkerIndex = 0; WorkerIndex < NumOpenColorIOShaderCompilingThreads; WorkerIndex++)
		{
			WorkerInfos.Add(new FOpenColorIOShaderCompileWorkerInfo());
		}
	}	
}

void FOpenColorIOShaderCompilationManager::AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs)
{
	check(IsInGameThread());
	JobQueue.Append(InNewJobs);

	for (FShaderCommonCompileJobPtr& Job : InNewJobs)
	{
		FOpenColorIOShaderMapCompileResults& ShaderMapInfo = OpenColorIOShaderMapJobs.FindOrAdd(Job->Id);
		ShaderMapInfo.NumJobsQueued++;

		auto CurrentJob = Job->GetSingleShaderJob();

		// Fast math breaks The ExecGrid layout script because floor(x/y) returns a bad value if x == y. Yay.
		if (IsMetalPlatform((EShaderPlatform)CurrentJob->Input.Target.Platform))
		{
			CurrentJob->Input.Environment.CompilerFlags.Add(CFLAG_NoFastMath);
		}

		CurrentJob->Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / CurrentJob->Input.ShaderPlatformName.ToString();
		FPaths::NormalizeDirectoryName(CurrentJob->Input.DumpDebugInfoRootPath);
		CurrentJob->Input.DebugExtension.Empty();
		CurrentJob->Input.DumpDebugInfoPath.Empty();
		if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
		{
			CurrentJob->Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(CurrentJob->Input);
		}
	}
	GShaderCompilingManager->SubmitJobs(InNewJobs, FString(), FString());
}

void FOpenColorIOShaderCompilationManager::ProcessAsyncResults()
{
	check(IsInGameThread());

	// Process the results from the shader compile worker
	for (int32 JobIndex = JobQueue.Num() - 1; JobIndex >= 0; JobIndex--)
	{
		auto CurrentJob = JobQueue[JobIndex]->GetSingleShaderJob();

		if (CurrentJob->GetStatus() != EShaderCompileJobStatus::Released)
		{
			continue;
		}

		CurrentJob->bSucceeded = CurrentJob->Output.bSucceeded;
		if (CurrentJob->Output.bSucceeded)
		{
			UE_LOGF(LogShaders, Verbose, "GPU shader compile succeeded. Id %d", CurrentJob->Id);
		}
		else
		{
			UE_LOGF(LogShaders, Warning, "GPU shader compile failed! Id: %d Name: %ls", CurrentJob->Id, *CurrentJob->Input.DebugGroupName);
		}

		FOpenColorIOShaderMapCompileResults& ShaderMapResults = OpenColorIOShaderMapJobs.FindChecked(CurrentJob->Id);
		ShaderMapResults.FinishedJobs.Add(JobQueue[JobIndex]);
		ShaderMapResults.bAllJobsSucceeded = ShaderMapResults.bAllJobsSucceeded && CurrentJob->bSucceeded;
		JobQueue.RemoveAt(JobIndex);
	}

	// Get all OCIO shader maps to finalize
	//
	for (TMap<int32, FOpenColorIOShaderMapCompileResults>::TIterator It(OpenColorIOShaderMapJobs); It; ++It)
	{
		const FOpenColorIOShaderMapCompileResults& Results = It.Value();

		if (Results.FinishedJobs.Num() == Results.NumJobsQueued)
		{
			PendingFinalizeOpenColorIOShaderMaps.Add(It.Key(), FOpenColorIOShaderMapCompileResults(Results));
			It.RemoveCurrent();
		}
	}

	if (PendingFinalizeOpenColorIOShaderMaps.Num() > 0)
	{
		ProcessCompiledOpenColorIOShaderMaps(PendingFinalizeOpenColorIOShaderMaps, 10);
	}
}

void FOpenColorIOShaderCompilationManager::ProcessCompiledOpenColorIOShaderMaps(
	TMap<int32, FOpenColorIOShaderMapFinalizeResults>& CompiledShaderMaps,
	float TimeBudget)
{
	// Keeps shader maps alive as they are passed from the shader compiler and applied to the owning ColorTransform
	TArray<TRefCountPtr<FOpenColorIOShaderMap> > LocalShaderMapReferences;
	TMap<FOpenColorIOTransformResource*, FOpenColorIOShaderMap*> TransformsToUpdate;

	// Process compiled shader maps in FIFO order, in case a shader map has been enqueued multiple times,
	// Which can happen if a ColorTransform is edited while a background compile is going on
	for (TMap<int32, FOpenColorIOShaderMapFinalizeResults>::TIterator ProcessIt(CompiledShaderMaps); ProcessIt; ++ProcessIt)
	{
		TRefCountPtr<FOpenColorIOShaderMap> ShaderMap = nullptr;
		TArray<FOpenColorIOTransformResource*>* ColorTransforms = nullptr;

		for (TMap<TRefCountPtr<FOpenColorIOShaderMap>, TArray<FOpenColorIOTransformResource*> >::TIterator ShaderMapIt(FOpenColorIOShaderMap::GetInFlightShaderMaps()); ShaderMapIt; ++ShaderMapIt)
		{
			if (ShaderMapIt.Key()->GetCompilingId() == ProcessIt.Key())
			{
				ShaderMap = ShaderMapIt.Key();
				ColorTransforms = &ShaderMapIt.Value();
				break;
			}
		}

		if (ShaderMap && ColorTransforms)
		{
			TArray<FString> Errors;
			FOpenColorIOShaderMapFinalizeResults& CompileResults = ProcessIt.Value();
			auto& ResultArray = CompileResults.FinishedJobs;

			// Make a copy of the array as this entry of FOpenColorIOShaderMap::ShaderMapsBeingCompiled will be removed below
			TArray<FOpenColorIOTransformResource*> ColorTransformArray = *ColorTransforms;
			bool bSuccess = true;

			for (int32 JobIndex = 0; JobIndex < ResultArray.Num(); JobIndex++)
			{
				FShaderCompileJob* CurrentJob = ResultArray[JobIndex]->GetSingleShaderJob();
				bSuccess = bSuccess && CurrentJob->bSucceeded;

				if (bSuccess)
				{
					check(CurrentJob->Output.ShaderCode.GetShaderCodeSize() > 0);
				}

				if (GShowOpenColorIOShaderWarnings || !CurrentJob->bSucceeded)
				{
					for (int32 ErrorIndex = 0; ErrorIndex < CurrentJob->Output.Errors.Num(); ErrorIndex++)
					{
						Errors.AddUnique(CurrentJob->Output.Errors[ErrorIndex].GetErrorString());
					}

					if (CurrentJob->Output.Errors.Num())
					{
						UE_LOGF(LogShaders, Log, "There were errors for job \"%ls\"", *CurrentJob->Input.DebugGroupName)
						for (const FShaderCompilerError& Error : CurrentJob->Output.Errors)
						{
							UE_LOGF(LogShaders, Log, "Error: %ls", *Error.GetErrorString())
						}
					}
				}
				else
				{
					UE_LOGF(LogShaders, Log, "There were NO errors for job \"%ls\"", *CurrentJob->Input.DebugGroupName);
				}
			}

			bool bShaderMapComplete = true;

			if (bSuccess)
			{
				bShaderMapComplete = ShaderMap->ProcessCompilationResults(ResultArray, CompileResults.FinalizeJobIndex, TimeBudget);
			}


			if (bShaderMapComplete)
			{
				ShaderMap->SetCompiledSuccessfully(bSuccess);

				// Pass off the reference of the shader map to LocalShaderMapReferences
				LocalShaderMapReferences.Add(ShaderMap);
				FOpenColorIOShaderMap::GetInFlightShaderMaps().Remove(ShaderMap);

				for (FOpenColorIOTransformResource* ColorTransform : ColorTransformArray)
				{
					FOpenColorIOShaderMap* CompletedShaderMap = ShaderMap;

					ColorTransform->RemoveOutstandingCompileId(ShaderMap->GetCompilingId());

					// Only process results that still match the ID which requested a compile
					// This avoids applying shadermaps which are out of date and a newer one is in the async compiling pipeline
					if (ColorTransform->IsSame(CompletedShaderMap->GetShaderMapId()))
					{
						if (Errors.Num() != 0)
						{
							FString SourceCode;
							ColorTransform->GetColorTransformHLSLSource(SourceCode);
							UE_LOGF(LogOpenColorIOShaderCompiler, Log, "Compile output as text:");
							UE_LOGF(LogOpenColorIOShaderCompiler, Log, "===================================================================================");
							TArray<FString> OutputByLines;
							SourceCode.ParseIntoArrayLines(OutputByLines, false);
							for (int32 i = 0; i < OutputByLines.Num(); i++)
							{
								UE_LOGF(LogOpenColorIOShaderCompiler, Log, "/*%04d*/\t\t%ls", i + 1, *OutputByLines[i]);
							}
							UE_LOGF(LogOpenColorIOShaderCompiler, Log, "===================================================================================");
						}

						if (!bSuccess)
						{
							// Propagate error messages
							ColorTransform->SetCompileErrors(Errors);
							TransformsToUpdate.Add(ColorTransform, nullptr);

							for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ErrorIndex++)
							{
								FString ErrorMessage = Errors[ErrorIndex];
								// Work around build machine string matching heuristics that will cause a cook to fail
								ErrorMessage.ReplaceInline(TEXT("error "), TEXT("err0r "), ESearchCase::CaseSensitive);
								UE_LOGF(LogOpenColorIOShaderCompiler, Warning, "	%ls", *ErrorMessage);
							}
						}
						else
						{
							// if we succeeded and our shader map is not complete this could be because the color transform was being edited quicker then the compile could be completed
							// Don't modify color transforms for which the compiled shader map is no longer complete
							// This shouldn't happen since transforms are pretty much baked in the designated config file.
							if (CompletedShaderMap->IsComplete(ColorTransform, true))
							{
								TransformsToUpdate.Add(ColorTransform, CompletedShaderMap);
							}

							if (GShowOpenColorIOShaderWarnings && Errors.Num() > 0)
							{
								UE_LOGF(LogOpenColorIOShaderCompiler, Warning, "Warnings while compiling OpenColorIO ColorTransform %ls for platform %ls:",
									*ColorTransform->GetFriendlyName(),
									*LegacyShaderPlatformToShaderFormat(ShaderMap->GetShaderPlatform()).ToString());
								for (int32 ErrorIndex = 0; ErrorIndex < Errors.Num(); ErrorIndex++)
								{
									UE_LOGF(LogOpenColorIOShaderCompiler, Warning, "	%ls", *Errors[ErrorIndex]);
								}
							}
						}
					}
				}

				// Cleanup shader jobs and compile tracking structures
				ResultArray.Empty();
				ProcessIt.RemoveCurrent();
			}

			if (TimeBudget < 0)
			{
				break;
			}
		}
	}

	if (TransformsToUpdate.Num() > 0)
	{
		for (TMap<FOpenColorIOTransformResource*, FOpenColorIOShaderMap*>::TConstIterator It(TransformsToUpdate); It; ++It)
		{
			FOpenColorIOTransformResource* ColorTransform = It.Key();
			FOpenColorIOShaderMap* ShaderMap = It.Value();

			ColorTransform->SetGameThreadShaderMap(It.Value());

			ENQUEUE_RENDER_COMMAND(FSetShaderMapOnColorTransformResources)(
				[ColorTransform, ShaderMap](FRHICommandListImmediate& RHICmdList)
				{
					ColorTransform->SetRenderingThreadShaderMap(ShaderMap);
				});
		}
	}
}

void FOpenColorIOShaderCompilationManager::FinishCompilation(const TCHAR* InTransformName, const TArray<int32>& ShaderMapIdsToFinishCompiling)
{
	check(!FPlatformProperties::RequiresCookedData());
	GShaderCompilingManager->FinishCompilation(NULL, ShaderMapIdsToFinishCompiling);
	ProcessAsyncResults();	// grab compiled shader maps and assign them to their resources
}

#endif // WITH_EDITOR
