// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOShader.h"

#include "Engine/VolumeTexture.h"
#include "OpenColorIOShared.h"
#include "OpenColorIODerivedDataVersion.h"
#include "OpenColorIOShaderCompilationManager.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
	#include "OpenColorIOWrapper.h"
	#include "DerivedDataCache.h"
	#include "DerivedDataRequestOwner.h"
	#include "Interfaces/ITargetPlatformManagerModule.h"
	#include "TickableEditorObject.h"
#endif


#if ENABLE_COOK_STATS
namespace OpenColorIOShaderCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static int32 ShadersCompiled = 0;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("OpenColorIOShader.Usage"), TEXT(""));
		AddStat(TEXT("OpenColorIOShader.Misc"), FCookStatsManager::CreateKeyValueArray(
			TEXT("ShadersCompiled"), ShadersCompiled
		));
	});
}
#endif


static TAutoConsoleVariable<bool> CVarShowOpenColorIONoShaderDDC(
	TEXT("OpenColorIO.NoShaderDDC"),
	false,
	TEXT("When enabled, we don't check the DDC for compiled shaders to simulate a cold DDC.")
);

//
// Globals
//
TMap<FOpenColorIOShaderMapId, FOpenColorIOShaderMap*> FOpenColorIOShaderMap::GIdToOpenColorIOShaderMap[SP_NumPlatforms];
TArray<FOpenColorIOShaderMap*> FOpenColorIOShaderMap::AllOpenColorIOShaderMaps;

#if WITH_EDITOR
// The Id of 0 is reserved for global shaders
uint32 FOpenColorIOShaderMap::NextCompilingId = 2;


/** 
 * Tracks FOpenColorIOTransformResource and their shader maps that are being compiled.
 * Uses a TRefCountPtr as this will be the only reference to a shader map while it is being compiled.
 */
TMap<TRefCountPtr<FOpenColorIOShaderMap>, TArray<FOpenColorIOTransformResource*> > FOpenColorIOShaderMap::OpenColorIOShaderMapsBeingCompiled;
#endif // WITH_EDITOR

static inline bool ShouldCacheOpenColorIOShader(const FOpenColorIOShaderType* InShaderType, EShaderPlatform InPlatform, const FOpenColorIOTransformResource* InColorTransformShader)
{
	return InShaderType->ShouldCache(InPlatform, InColorTransformShader) && InColorTransformShader->ShouldCache(InPlatform, InShaderType);
}

/** Called for every color transform shader to update the appropriate stats. */
void UpdateOpenColorIOShaderCompilingStats(const FOpenColorIOTransformResource* InShader)
{
	INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumTotalOpenColorIOShaders,1);
}

/*void FOpenColorIOShaderMapId::Serialize(FArchive& Ar)
{
	// You must bump OPENCOLORIO_DERIVEDDATA_VER if changing the serialization of FOpenColorIOShaderMapId.

	Ar << ShaderCodeHash;
	Ar << (int32&)FeatureLevel;
	Ar << ShaderTypeDependencies;
}*/

/** Hashes the color transform specific part of this shader map Id. */
uint64 FOpenColorIOShaderMapId::GetOpenColorIOHash() const
{
	FXxHash64Builder HashState;

	HashState.Update(*ShaderCodeAndConfigHash, ShaderCodeAndConfigHash.Len() * sizeof(FMemoryImageString::ElementType));
	HashState.Update((const uint8*)&FeatureLevel, sizeof(FeatureLevel));

	return HashState.Finalize().Hash;
}

/** 
* Tests this set against another for equality, disregarding override settings.
* 
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FOpenColorIOShaderMapId::operator==(const FOpenColorIOShaderMapId& InReferenceSet) const
{
	if (  ShaderCodeAndConfigHash != InReferenceSet.ShaderCodeAndConfigHash
		|| FeatureLevel != InReferenceSet.FeatureLevel)
	{
		return false;
	}

	if (ShaderTypeDependencies.Num() != InReferenceSet.ShaderTypeDependencies.Num())
	{
		return false;
	}

	if (LayoutParams != InReferenceSet.LayoutParams)
	{
		return false;
	}

	for (int32 ShaderIndex = 0; ShaderIndex < ShaderTypeDependencies.Num(); ShaderIndex++)
	{
		const FShaderTypeDependency& ShaderTypeDependency = ShaderTypeDependencies[ShaderIndex];

		if (ShaderTypeDependency != InReferenceSet.ShaderTypeDependencies[ShaderIndex])
		{
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR

void FOpenColorIOShaderMapId::AppendKeyString(FString& OutKeyString) const
{
	OutKeyString.Append(ShaderCodeAndConfigHash);
	OutKeyString.AppendChar('_');

	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FOpenColorIOShaderMapContent>(), LayoutParams);
		OutKeyString.AppendChar('_');
		LayoutHash.AppendString(OutKeyString);
		OutKeyString.AppendChar('_');
	}

	// Add the inputs for any shaders that are stored inline in the shader map
	AppendKeyStringShaderDependencies(MakeArrayView(ShaderTypeDependencies), LayoutParams, OutKeyString);
}

void FOpenColorIOShaderType::BeginCompileShader(
	uint32 InShaderMapId,
	const FOpenColorIOTransformResource* InColorTransform,
	FSharedShaderCompilerEnvironment* InCompilationEnvironment,
	EShaderPlatform InPlatform,
	TArray<FShaderCommonCompileJobPtr>& OutNewJobs,
	FShaderTarget InTarget
	)
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(InShaderMapId, FShaderCompileJobKey(this, InPlatform), EShaderCompileJobPriority::Normal);
	if (!NewJob)
	{
		return;
	}
	NewJob->ShaderParameters = MakeShared<const FParameters, ESPMode::ThreadSafe>(InColorTransform->GetShaderParamMetadata().ToSharedRef());
	NewJob->Input.SharedEnvironment = InCompilationEnvironment;
	NewJob->Input.Target = InTarget;
	NewJob->Input.ShaderFormat = LegacyShaderPlatformToShaderFormat(InPlatform);
	NewJob->Input.VirtualSourceFilePath = TEXT("/Engine/Plugins/Compositing/OpenColorIO/Shaders/Private/OpenColorIOShader.usf");
	NewJob->Input.EntryPointName = TEXT("MainPS");
	NewJob->Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/OpenColorIOTransformShader.ush"), InColorTransform->ShaderCode);

	AddUniformBufferIncludesToEnvironment(NewJob->Input.Environment, InPlatform);

	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	//update InColorTransform shader stats
	UpdateOpenColorIOShaderCompilingStats(InColorTransform);

	// Allow the shader type to modify the compile environment.
	SetupCompileEnvironment(InPlatform, InColorTransform, ShaderEnvironment);

	::GlobalBeginCompileShader(
		InColorTransform->GetFriendlyName(),
		nullptr,
		this,
		nullptr,//ShaderPipeline
		0, // PermutationId
		TEXT("/Plugin/OpenColorIO/Private/OpenColorIOShader.usf"),
		TEXT("MainPS"),
		FShaderTarget(GetFrequency(), InPlatform),
		NewJob->Input
	);

	OutNewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
}

FShader* FOpenColorIOShaderType::FinishCompileShader(
	uint64 InShaderMapHash,
	const FShaderCompileJob& InCurrentJob,
	const FString& InDebugDescription
	) const
{
	check(InCurrentJob.bSucceeded);

	FShader* Shader = ConstructCompiled(
		FOpenColorIOShaderType::CompiledShaderInitializerType(
			this,
			static_cast<const FParameters*>(InCurrentJob.ShaderParameters.Get()),
			InCurrentJob.Key.PermutationId,
			InCurrentJob.Output,
			InShaderMapHash,
			InDebugDescription
		)
	);
	InCurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(GetName(), InCurrentJob.Output.Target, nullptr);

	return Shader;
}

void FOpenColorIOShaderType::SetupCompileEnvironment(EShaderPlatform InPlatform, const FOpenColorIOTransformResource* InColorTransform, FShaderCompilerEnvironment& OutEnvironment) const
{
	ModifyCompilationEnvironment(FOpenColorIOShaderPermutationParameters(GetFName(), InPlatform, InColorTransform), OutEnvironment);
}

#endif // WITH_EDITOR

/**
 * Finds the shader map for a color transform.
 * @param InShaderMapId - The color transform id and static parameter set identifying the shader map
 * @param InPlatform - The platform to lookup for
 * @return nullptr if no cached shader map was found.
 */
FOpenColorIOShaderMap* FOpenColorIOShaderMap::FindId(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform)
{
	check(!InShaderMapId.ShaderCodeAndConfigHash.IsEmpty());
	return GIdToOpenColorIOShaderMap[InPlatform].FindRef(InShaderMapId);
}

#if WITH_EDITOR

void OpenColorIOShaderMapAppendKeyString(EShaderPlatform InPlatform, FString& OutKeyString)
{
	//Keep library version in the DDC key to invalidate it once we move to a new library
	OutKeyString += TEXT("OCIOVersion");
	OutKeyString += OpenColorIOWrapper::GetVersion();
	OutKeyString += TEXT("_");
}

/** Creates a string key for the derived data cache given a shader map id. */
static UE::DerivedData::FCacheKey GetOpenColorIOShaderMapKeyString(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform)
{
	const FName Format = LegacyShaderPlatformToShaderFormat(InPlatform);
	FString ShaderMapKeyString = Format.ToString() + TEXT("_") + FString(FString::FromInt(GetTargetPlatformManagerRef().ShaderFormatVersion(Format))) + TEXT("_");
	OpenColorIOShaderMapAppendKeyString(InPlatform, ShaderMapKeyString);
	ShaderMapAppendKeyString(InPlatform, ShaderMapKeyString);
	InShaderMapId.AppendKeyString(ShaderMapKeyString);
	return UE::DerivedData::BuildLegacyCacheKey(TEXT("OCIOSM"), OPENCOLORIO_DERIVEDDATA_VER, ShaderMapKeyString);
}

void FOpenColorIOShaderMap::LoadFromDerivedDataCache(const FOpenColorIOTransformResource* InColorTransform, const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FOpenColorIOShaderMap>& InOutShaderMap)
{
	if (InOutShaderMap != nullptr)
	{
		check(InOutShaderMap->GetShaderPlatform() == InPlatform);
	}
	else
	{
		static bool bNoShaderDDC = FParse::Param(FCommandLine::Get(), TEXT("noshaderddc"));

		if (bNoShaderDDC || CVarShowOpenColorIONoShaderDDC.GetValueOnAnyThread())
		{
			return;
		}

		// Shader map was not found in memory, try to load it from the DDC
		STAT(double OpenColorIOShaderDDCTime = 0);
		{
			SCOPE_SECONDS_COUNTER(OpenColorIOShaderDDCTime);
			COOK_STAT(auto Timer = OpenColorIOShaderCookStats::UsageStats.TimeSyncWork());

			using namespace UE::DerivedData;
			const FCacheKey Key = GetOpenColorIOShaderMapKeyString(InShaderMapId, InPlatform);
			FRequestOwner Owner(EPriority::Blocking);
			GetCache().GetValue({{{InColorTransform->GetFriendlyName()}, Key}}, Owner, [InShaderMapId, InPlatform, &InOutShaderMap, &Timer](FCacheGetValueResponse&& Response)
			{
				if (Response.Status == EStatus::Ok)
				{
					COOK_STAT(Timer.AddHit(Response.Value.GetRawSize()));

					FSharedBuffer CachedData = Response.Value.GetData().Decompress();

					InOutShaderMap = new FOpenColorIOShaderMap();
					FMemoryReaderView Ar(CachedData, true);

					// Deserialize from the cached data
					InOutShaderMap->Serialize(Ar);

					checkSlow(InOutShaderMap->GetShaderMapId() == InShaderMapId);

					// Register in the global map
					InOutShaderMap->Register(InPlatform);
				}
				else
				{
					// We should be build the data later, and we can track that the resource was built there when we push it to the DDC.
					COOK_STAT(Timer.TrackCyclesOnly());
					InOutShaderMap = nullptr;
				}
			});
			Owner.Wait();
		}
		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_DDCLoading,(float)OpenColorIOShaderDDCTime);
	}
}

void FOpenColorIOShaderMap::SaveToDerivedDataCache()
{
	using namespace UE::DerivedData;

	COOK_STAT(auto Timer = OpenColorIOShaderCookStats::UsageStats.TimeSyncWork());
	FValue Value;
	{
		TArray<uint8> SaveData;
		FMemoryWriter Ar(SaveData, true);
		Serialize(Ar);
		Value = FValue::Compress(MakeSharedBufferFromArray(MoveTemp(SaveData)));
	}

	const FCacheKey Key = GetOpenColorIOShaderMapKeyString(GetContent()->ShaderMapId, GetShaderPlatform());
	FRequestOwner Owner(EPriority::Normal);
	GetCache().PutValue({{{GetFriendlyName()}, Key, Value}}, Owner);
	Owner.KeepAlive();
	COOK_STAT(Timer.AddMiss(Value.GetRawSize()));
}

/**
* Compiles the shaders for a color transform and caches them in this shader map.
* @param InColorTransform - The ColorTransform to compile shaders for.
* @param InShaderMapId - the color transform id and set of static parameters to compile for
* @param InPlatform - The platform to compile to
*/
void FOpenColorIOShaderMap::Compile(FOpenColorIOTransformResource* InColorTransform
									, const FOpenColorIOShaderMapId& InShaderMapId
									, TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment
									, const FOpenColorIOCompilationOutput& InOpenColorIOCompilationOutput
									, EShaderPlatform InPlatform
									, bool bSynchronousCompile
									, bool bApplyCompletedShaderMapForRendering)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UE_LOGF(LogShaders, Fatal, "Trying to compile OpenColorIO shader %ls at run-time, which is not supported on consoles!", *InColorTransform->GetFriendlyName());
	}
	else
	{
		// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
		// Since it creates a temporary ref counted pointer.
		check(NumRefs > 0);
  
		//All access to OpenColorIOShaderMapsBeingCompiled must be done on the game thread!
		check(IsInGameThread());
		// Add this shader map and to OpenColorIOShaderMapsBeingCompiled
		TArray<FOpenColorIOTransformResource*>* CorrespondingTransform = OpenColorIOShaderMapsBeingCompiled.Find(this);
  
		if (CorrespondingTransform)
		{
			check(!bSynchronousCompile);
			CorrespondingTransform->AddUnique(InColorTransform);
		}
		else
		{
			InColorTransform->RemoveOutstandingCompileId(CompilingId);
			// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
			CompilingId = FShaderCommonCompileJob::GetNextJobId();
			InColorTransform->AddCompileId(CompilingId);
  
			TArray<FOpenColorIOTransformResource*> NewCorrespondingTransforms;
			NewCorrespondingTransforms.Add(InColorTransform);
			OpenColorIOShaderMapsBeingCompiled.Add(this, NewCorrespondingTransforms);
			// Setup the compilation environment.
			InColorTransform->SetupShaderCompilationEnvironment(InPlatform, *InCompilationEnvironment);
  
			// Store the ColorTransform name for debugging purposes.
			FOpenColorIOShaderMapContent* NewContent = new FOpenColorIOShaderMapContent(InPlatform);
			NewContent->FriendlyName = InColorTransform->GetFriendlyName();
			NewContent->OpenColorIOCompilationOutput = InOpenColorIOCompilationOutput;
			NewContent->ShaderMapId = InShaderMapId;
			AssignContent(NewContent);

			uint32 NumShaders = 0;
			TArray<FShaderCommonCompileJobPtr> NewJobs;
	
			// Iterate over all shader types.
			for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
			{
				FOpenColorIOShaderType* ShaderType = ShaderTypeIt->GetOpenColorIOShaderType();
				if (ShaderType && ShouldCacheOpenColorIOShader(ShaderType, InPlatform, InColorTransform))
				{
					// Compile this OpenColorIO shader .
					TArray<FString> ShaderErrors;
  
					// Only compile the shader if we don't already have it
					if (!NewContent->HasShader(ShaderType, /* PermutationId = */ 0))
					{
						ShaderType->BeginCompileShader(
							CompilingId,
							InColorTransform,
							InCompilationEnvironment,
							InPlatform,
							NewJobs,
							FShaderTarget(ShaderType->GetFrequency(), GetShaderPlatform())
						);
					}
					NumShaders++;
				}
				else if (ShaderType)
				{
					UE_LOGF(LogShaders, Display, "Skipping compilation of %ls as it isn't supported on this target type.", *InColorTransform->GetFriendlyName());
					InColorTransform->RemoveOutstandingCompileId(CompilingId);
				}
			}
  
			if (!CorrespondingTransform)
			{
				UE_LOGF(LogShaders, Log, "		%u Shaders", NumShaders);
			}

			// Register this shader map in the global script->shadermap map
			Register(InPlatform);
  
			// Mark the shader map as not having been finalized with ProcessCompilationResults
			bCompilationFinalized = false;
  
			// Mark as not having been compiled
			bCompiledSuccessfully = false;
  
			GOpenColorIOShaderCompilationManager.AddJobs(NewJobs);
  
			// Compile the shaders for this shader map now if not deferring and deferred compiles are not enabled globally
			if (bSynchronousCompile)
			{
				TArray<int32> CurrentShaderMapId;
				CurrentShaderMapId.Add(CompilingId);
				GOpenColorIOShaderCompilationManager.FinishCompilation(*NewContent->FriendlyName, CurrentShaderMapId);
			}
		}
	}
}

FShader* FOpenColorIOShaderMap::ProcessCompilationResultsForSingleJob(const TRefCountPtr<class FShaderCommonCompileJob>& SingleJob, uint64 InShaderMapHash)
{
	FShaderCompileJob* CurrentJob = SingleJob->GetSingleShaderJob();
	check(CurrentJob->Id == CompilingId);

	GetResourceCode()->AddShaderCompilerOutput(CurrentJob->Output, CurrentJob->Key, CurrentJob->Input.GenerateDebugInfo());

	FShader* Shader = nullptr;

	const FOpenColorIOShaderType* OpenColorIOShaderType = CurrentJob->Key.ShaderType->GetOpenColorIOShaderType();
	check(OpenColorIOShaderType);
	Shader = OpenColorIOShaderType->FinishCompileShader(InShaderMapHash, *CurrentJob, GetContent()->FriendlyName);
	bCompiledSuccessfully = CurrentJob->bSucceeded;

	check(Shader && Shader->GetCodeSize() > 0);
	check(!GetContent()->HasShader(OpenColorIOShaderType, /* PermutationId = */ 0));
	return GetMutableContent()->FindOrAddShader(OpenColorIOShaderType->GetHashedName(), CurrentJob->Key.PermutationId, Shader);
}

bool FOpenColorIOShaderMap::ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutJobIndex, float& InOutTimeBudget)
{
	check(InOutJobIndex < InCompilationResults.Num());

	double StartTime = FPlatformTime::Seconds();

	uint64 ShaderMapHash = GetContent()->ShaderMapId.GetOpenColorIOHash();

	do
	{
		ProcessCompilationResultsForSingleJob(InCompilationResults[InOutJobIndex], ShaderMapHash);

		InOutJobIndex++;
		
		double NewStartTime = FPlatformTime::Seconds();
		InOutTimeBudget -= NewStartTime - StartTime;
		StartTime = NewStartTime;
	}
	while ((InOutTimeBudget > 0.0f) && (InOutJobIndex < InCompilationResults.Num()));

	if (InOutJobIndex == InCompilationResults.Num())
	{
		FinalizeContent();

		SaveToDerivedDataCache();
		// The shader map can now be used on the rendering thread
		bCompilationFinalized = true;
		return true;
	}

	return false;
}

bool FOpenColorIOShaderMap::TryToAddToExistingCompilationTask(FOpenColorIOTransformResource* InColorTransform)
{
	check(NumRefs > 0);
	TArray<FOpenColorIOTransformResource*>* CorrespondingColorTransforms = FOpenColorIOShaderMap::OpenColorIOShaderMapsBeingCompiled.Find(this);

	if (CorrespondingColorTransforms)
	{
		CorrespondingColorTransforms->AddUnique(InColorTransform);

		UE_LOGF(LogShaders, Log, "TryToAddToExistingCompilationTask %p %d", InColorTransform, GetCompilingId());

		return true;
	}

	return false;
}

#endif // WITH_EDITOR

bool FOpenColorIOShaderMap::IsOpenColorIOShaderComplete(const FOpenColorIOTransformResource* InColorTransform, const FOpenColorIOShaderType* InShaderType, bool bSilent)
{
	// If we should cache this color transform, it's incomplete if the shader is missing
	if (ShouldCacheOpenColorIOShader(InShaderType, GetShaderPlatform(), InColorTransform) && !GetContent()->HasShader((FShaderType*)InShaderType, /* PermutationId = */ 0))
	{
		if (!bSilent)
		{
			UE_LOGF(LogShaders, Warning, "Incomplete shader %ls, missing FOpenColorIOShader %ls.", *InColorTransform->GetFriendlyName(), InShaderType->GetName());
		}
		return false;
	}

	return true;
}

bool FOpenColorIOShaderMap::IsComplete(const FOpenColorIOTransformResource* InColorTransform, bool bSilent)
{
	// Make sure we are operating on a referenced shader map or the below Find will cause this shader map to be deleted,
	// Since it creates a temporary ref counted pointer.
	check(NumRefs > 0);

#if WITH_EDITOR
	if (const TArray<FOpenColorIOTransformResource*>* CorrespondingColorTransforms = FOpenColorIOShaderMap::OpenColorIOShaderMapsBeingCompiled.Find(this))
	{
		check(!bCompilationFinalized);
		return false;
	}
#endif // WITH_EDITOR

	// Iterate over all shader types.
	for(TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList());ShaderTypeIt;ShaderTypeIt.Next())
	{
		// Find this shader type in the ColorTransform's shader map.
		const FOpenColorIOShaderType* ShaderType = ShaderTypeIt->GetOpenColorIOShaderType();
		if (ShaderType && !IsOpenColorIOShaderComplete(InColorTransform, ShaderType, bSilent))
		{
			return false;
		}
	}

	return true;
}

void FOpenColorIOShaderMap::GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, 0u, OutShaders);
}

void FOpenColorIOShaderMap::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
}

void FOpenColorIOShaderMap::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
}

/**
 * Registers a OpenColorIO shader map in the global map.
 */
void FOpenColorIOShaderMap::Register(EShaderPlatform InShaderPlatform)
{
	if (!bRegistered)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShaderMaps);
	}

	GIdToOpenColorIOShaderMap[GetShaderPlatform()].Add(GetContent()->ShaderMapId,this);
	bRegistered = true;
}

void FOpenColorIOShaderMap::AddRef()
{
	check(!bDeletedThroughDeferredCleanup);
	++NumRefs;
}

void FOpenColorIOShaderMap::Release()
{
	check(NumRefs > 0);
	if(--NumRefs == 0)
	{
		if (bRegistered)
		{
			DEC_DWORD_STAT(STAT_Shaders_NumShaderMaps);

			GIdToOpenColorIOShaderMap[GetShaderPlatform()].Remove(GetContent()->ShaderMapId);
			bRegistered = false;
		}

		check(!bDeletedThroughDeferredCleanup);
		bDeletedThroughDeferredCleanup = true;
		BeginCleanup(this);
	}
}

FOpenColorIOShaderMap::FOpenColorIOShaderMap() :
	CompilingId(1),
	NumRefs(0),
	bDeletedThroughDeferredCleanup(false),
	bRegistered(false),
	bCompilationFinalized(true),
	bCompiledSuccessfully(true),
	bIsPersistent(true) 
{
	checkSlow(IsInGameThread() || IsAsyncLoading());
	AllOpenColorIOShaderMaps.Add(this);
}

FOpenColorIOShaderMap::~FOpenColorIOShaderMap()
{ 
	checkSlow(IsInGameThread() || IsAsyncLoading());
	check(bDeletedThroughDeferredCleanup);
	check(!bRegistered);
	AllOpenColorIOShaderMaps.RemoveSwap(this);
}

bool FOpenColorIOShaderMap::Serialize(FArchive& Ar)
{
	// Note: This is saved to the DDC, not into packages (except when cooked)
	// Backwards compatibility therefore will not work based on the version of Ar
	// Instead, just bump OPENCOLORIO_DERIVEDDATA_VER
	FShaderSerializeContext Ctx(Ar);
	return Super::Serialize(Ctx);
}

#if WITH_EDITOR
void FOpenColorIOShaderMap::RemovePendingColorTransform(FOpenColorIOTransformResource* InColorTransform)
{
	for (TMap<TRefCountPtr<FOpenColorIOShaderMap>, TArray<FOpenColorIOTransformResource*> >::TIterator It(OpenColorIOShaderMapsBeingCompiled); It; ++It)
	{
		TArray<FOpenColorIOTransformResource*>& ColorTranforms = It.Value();
		int32 Result = ColorTranforms.Remove(InColorTransform);
		if (Result)
		{
			InColorTransform->RemoveOutstandingCompileId(It.Key()->CompilingId);
		}
	}
}
#endif // WITH_EDITOR
