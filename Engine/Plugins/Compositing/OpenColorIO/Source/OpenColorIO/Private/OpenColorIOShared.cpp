// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenColorIOShared.cpp: Shared OpenColorIO pixel shader implementation.
=============================================================================*/

#include "OpenColorIOShared.h"

#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOModule.h"
#include "OpenColorIOShaderType.h"
#include "OpenColorIOShader.h"
#include "OpenColorIOShaderCompilationManager.h"
#include "RendererInterface.h"
#include "ShaderCompiler.h"
#include "Stats/StatsMisc.h"
#include "TextureResource.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Interfaces/ITargetPlatform.h"
#include "MaterialShared.h"

IMPLEMENT_TYPE_LAYOUT(FOpenColorIOCompilationOutput);
IMPLEMENT_TYPE_LAYOUT(FOpenColorIOShaderMapId);
IMPLEMENT_TYPE_LAYOUT(FOpenColorIOShaderMapContent);

FOpenColorIOTransformResource::FOpenColorIOTransformResource()
	: GameThreadShaderMap(nullptr)
	, RenderingThreadShaderMap(nullptr)
	, FeatureLevel(ERHIFeatureLevel::SM5)
	, bContainsInlineShaders(false)
	, bLoadedCookedShaderMapId(false)
{}

FOpenColorIOTransformResource::~FOpenColorIOTransformResource()
{
#if WITH_EDITOR
	FOpenColorIOShaderMap::RemovePendingColorTransform(this);
#endif
}

#if WITH_EDITOR
/** Populates OutEnvironment with defines needed to compile shaders for this color transform. */
void FOpenColorIOTransformResource::SetupShaderCompilationEnvironment(EShaderPlatform InPlatform, FShaderCompilerEnvironment& OutEnvironment) const
{
}
#endif // WITH_EDITOR

OPENCOLORIO_API bool FOpenColorIOTransformResource::ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const
{
	check(InShaderType->GetOpenColorIOShaderType() )
	return true;
}

#if WITH_EDITOR

OPENCOLORIO_API void FOpenColorIOTransformResource::CancelCompilation()
{
	if (IsInGameThread())
	{
		FOpenColorIOShaderMap::RemovePendingColorTransform(this);

		UE_LOGF(LogShaders, Log, "CancelCompilation %p.", this);
		OutstandingCompileShaderMapIds.Empty();
	}
}

OPENCOLORIO_API void FOpenColorIOTransformResource::RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId)
{
	if (0 <= OutstandingCompileShaderMapIds.Remove(InOldOutstandingCompileShaderMapId))
	{
		UE_LOGF(LogShaders, Log, "RemoveOutstandingCompileId %p %d", this, InOldOutstandingCompileShaderMapId);
	}
}

#endif // WITH_EDITOR

OPENCOLORIO_API void FOpenColorIOTransformResource::Invalidate()
{
#if WITH_EDITOR
	CancelCompilation();
#endif
	ReleaseShaderMap();
}

bool FOpenColorIOTransformResource::IsSame(const FOpenColorIOShaderMapId& InIdentifier) const
{
	return InIdentifier.ShaderCodeAndConfigHash == ShaderCodeAndConfigHash;
}

void FOpenColorIOTransformResource::GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const
{
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FOpenColorIOShaderType* ShaderType = ShaderTypeIt->GetOpenColorIOShaderType();

		if ( ShaderType && ShaderType->ShouldCache(InPlatform, this) && ShouldCache(InPlatform, ShaderType) )
		{
			OutShaderTypes.Add(ShaderType);
		}
	}

	OutShaderTypes.Sort(FCompareShaderTypes());
}

OPENCOLORIO_API void FOpenColorIOTransformResource::GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FOpenColorIOShaderMapId& OutId) const
{
	if (bLoadedCookedShaderMapId)
	{
		OutId = CookedShaderMapId;
	}
	else
	{
		TArray<FShaderType*> ShaderTypes;
		GetDependentShaderTypes(InPlatform, ShaderTypes);

		OutId.FeatureLevel = GetFeatureLevel();

		OutId.ShaderCodeAndConfigHash = ShaderCodeAndConfigHash;
#if WITH_EDITOR
		OutId.SetShaderDependencies(ShaderTypes, InPlatform);
		OutId.LayoutParams.InitializeForPlatform(TargetPlatform);
#else
		if (TargetPlatform != nullptr)
		{
			UE_LOGF(LogShaders, Error, "FOpenColorIOTransformResource::GetShaderMapId: TargetPlatform is not null, but a cooked executable cannot target platforms other than its own.");
		}
		OutId.LayoutParams.InitializeForCurrent();
#endif
	}
}

void FOpenColorIOTransformResource::ReleaseShaderMap()
{
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap = nullptr;

		FOpenColorIOTransformResource* ColorTransform = this;
		ENQUEUE_RENDER_COMMAND(ReleaseShaderMap)(
			[ColorTransform](FRHICommandListImmediate& RHICmdList)
			{
				ColorTransform->SetRenderingThreadShaderMap(nullptr);
			});
	}
}

void FOpenColorIOTransformResource::DiscardShaderMap()
{
	check(RenderingThreadShaderMap == nullptr);
	GameThreadShaderMap = nullptr;
}

void FOpenColorIOTransformResource::SerializeShaderMap(FArchive& Ar)
{
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogShaders, Fatal, "This platform requires cooked packages, and shaders were not cooked into this OCIO transform %ls.", *GetFriendlyName());
	}

	if (bCooked)
	{
		if (Ar.IsCooking())
		{
#if WITH_EDITOR
			FinishCompilation();

			bool bValid = GameThreadShaderMap != nullptr && GameThreadShaderMap->CompiledSuccessfully();
			Ar << bValid;

			if (bValid)
			{
				GameThreadShaderMap->Serialize(Ar);
			}
			//else if (GameThreadShaderMap != nullptr && !GameThreadShaderMap->CompiledSuccessfully())
			//{
			//	FString Name;
			//	UE_LOGF(LogShaders, Error, "Failed to compile OpenColorIO shader %ls.", *GetFriendlyName());
			//}
#endif
		}
		else
		{
			bool bValid = false;
			Ar << bValid;

			if (bValid)
			{
				TRefCountPtr<FOpenColorIOShaderMap> LoadedShaderMap = new FOpenColorIOShaderMap();
				bool bSuccessfullyLoaded = LoadedShaderMap->Serialize(Ar);

				// Toss the loaded shader data if this is a server only instance (@todo - don't cook it in the first place) or if it's for a different RHI than the current one
				if (bSuccessfullyLoaded && FApp::CanEverRender())
				{
					GameThreadShaderMap = RenderingThreadShaderMap = LoadedShaderMap;
					GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
#if WITH_EDITOR
					GameThreadShaderMap->AssociateWithAsset(AssetPath);
#endif
				}
			}
		}
	}
}

void FOpenColorIOTransformResource::SetupResource(ERHIFeatureLevel::Type InFeatureLevel, const FString& InShaderCodeHash, const FString& InShadercode, const FString& InRawConfigHash, const FString& InFriendlyName, const FName& InAssetPath, FShaderParametersMetadata* InShaderParameterMetadata)
{
	// Take ownership of the parameter metadata
	ShaderParametersMetadata = TSharedPtr<FShaderParametersMetadata>(InShaderParameterMetadata);

	// When this happens we assume that shader was cooked and we don't need to do anything else.
	if (InShaderCodeHash.IsEmpty() && InRawConfigHash.IsEmpty())
	{
		return;
	}

	const FString ConcatHashes = InShaderCodeHash + InRawConfigHash;
	FSHAHash FullHash;
	FSHA1::HashBuffer(TCHAR_TO_ANSI(*(ConcatHashes)), ConcatHashes.Len(), FullHash.Hash);

	ShaderCodeAndConfigHash = FullHash.ToString();
	ShaderCode = InShadercode;
	FriendlyName = InFriendlyName.Replace(TEXT("/"), TEXT(""));
	AssetPath = InAssetPath;

	if (!ShaderParametersMetadata.IsValid())
	{
		UE_LOGF(LogOpenColorIO, Error, "OpenColorIO shader transform resource is expecting a valid shader parameter metadata for compilation.");
	}

	SetFeatureLevel(InFeatureLevel);
}

OPENCOLORIO_API  void FOpenColorIOTransformResource::SetRenderingThreadShaderMap(FOpenColorIOShaderMap* InShaderMap)
{
	check(IsInRenderingThread());
	RenderingThreadShaderMap = InShaderMap;
}

OPENCOLORIO_API  bool FOpenColorIOTransformResource::IsCompilationFinished() const
{
#if WITH_EDITOR
	if (OutstandingCompileShaderMapIds.Num() == 0)
	{
		return true;
	}
#endif
	bool bRet = GameThreadShaderMap && GameThreadShaderMap.IsValid() && GameThreadShaderMap->IsCompilationFinalized();
	return bRet;
}

bool FOpenColorIOTransformResource::CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	FOpenColorIOShaderMapId ResourceShaderMapId;
	GetShaderMapId(InPlatform, TargetPlatform, ResourceShaderMapId);
	return CacheShaders(ResourceShaderMapId, InPlatform, bApplyCompletedShaderMapForRendering, bSynchronous);
}

bool FOpenColorIOTransformResource::CacheShaders(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	bool bSucceeded = false;
	{
		// If we loaded this material with inline shaders, use what was loaded (GameThreadShaderMap) instead of looking in the DDC
		if (bContainsInlineShaders)
		{
			FOpenColorIOShaderMap* ExistingShaderMap = nullptr;

			if (GameThreadShaderMap)
			{
				// Note: in the case of an inlined shader map, the shadermap Id will not be valid because we stripped some editor-only data needed to create it
				// Get the shadermap Id from the shadermap that was inlined into the package, if it exists
				ExistingShaderMap = FOpenColorIOShaderMap::FindId(GameThreadShaderMap->GetShaderMapId(), InPlatform);
			}

			// Re-use an identical shader map in memory if possible, removing the reference to the inlined shader map
			if (ExistingShaderMap)
			{
				GameThreadShaderMap = ExistingShaderMap;
			}
			else if (GameThreadShaderMap)
			{
				// We are going to use the inlined shader map, register it so it can be re-used by other materials
				GameThreadShaderMap->Register(InPlatform);
			}
		}
		else
		{
			// Find the color transform's cached shader map.
			GameThreadShaderMap = FOpenColorIOShaderMap::FindId(InShaderMapId, InPlatform);

			// Attempt to load from the derived data cache if we are uncooked
			if ((!GameThreadShaderMap || !GameThreadShaderMap->IsComplete(this, true)) && !FPlatformProperties::RequiresCookedData())
			{
#if WITH_EDITOR
				FOpenColorIOShaderMap::LoadFromDerivedDataCache(this, InShaderMapId, InPlatform, GameThreadShaderMap);
				if (GameThreadShaderMap && GameThreadShaderMap->IsValid())
				{
					UE_LOGF(LogOpenColorIO, Display, "Loaded shader %ls for transform %ls from DDC", *GameThreadShaderMap->GetFriendlyName(), *GetFriendlyName());
				}
				else
				{
					UE_LOGF(LogOpenColorIO, Display, "Shader for transform %ls was not loaded from the DDC, needs recompile.", *GetFriendlyName());
				}
#endif // WITH_EDITOR
			}
		}
	}

	bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAssumeShaderMapIsComplete = (bContainsInlineShaders || FPlatformProperties::RequiresCookedData());
#endif

#if WITH_EDITOR
	// maintain asset association for newly loaded shader maps
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap->AssociateWithAsset(AssetPath);
	}

	if (GameThreadShaderMap && GameThreadShaderMap->TryToAddToExistingCompilationTask(this))
	{
		OutstandingCompileShaderMapIds.AddUnique(GameThreadShaderMap->GetCompilingId());
		UE_LOGF(LogShaders, Log, "CacheShaders AddUniqueExisting %p %d", this, GameThreadShaderMap->GetCompilingId());

		GameThreadShaderMap = nullptr;
		bSucceeded = true;
	}
	else
#endif // WITH_EDITOR
	if (!GameThreadShaderMap || !(bAssumeShaderMapIsComplete || GameThreadShaderMap->IsComplete(this, false)))
	{
		if (bContainsInlineShaders || FPlatformProperties::RequiresCookedData())
		{
			UE_LOGF(LogShaders, Log, "Can't compile %ls with cooked content!", *GetFriendlyName());
			GameThreadShaderMap = nullptr;
		}
#if WITH_EDITOR
		else
		{
			UE_LOGF(LogShaders, Log, "%ls cached shader map for color transform %ls, compiling.", GameThreadShaderMap? TEXT("Incomplete") : TEXT("Missing"), *GetFriendlyName());

			// If there's no cached shader map for this color transform compile a new one.
			// This is just kicking off the compile, GameThreadShaderMap will not be complete yet
			bSucceeded = BeginCompileShaderMap(InShaderMapId, InPlatform, GameThreadShaderMap, bApplyCompletedShaderMapForRendering, bSynchronous);

			if (!bSucceeded)
			{
				GameThreadShaderMap = nullptr;
			}
		}
#endif // WITH_EDITOR
	}
	else
	{
		bSucceeded = true;
	}

	FOpenColorIOTransformResource* ColorSpaceTransform = this;
	FOpenColorIOShaderMap* LoadedShaderMap = GameThreadShaderMap;
	ENQUEUE_RENDER_COMMAND(FSetShaderMapOnColorSpaceTransformResources)(
		[ColorSpaceTransform, LoadedShaderMap](FRHICommandListImmediate& RHICmdList)
		{
			ColorSpaceTransform->SetRenderingThreadShaderMap(LoadedShaderMap);
		});

	return bSucceeded;
}

TSharedPtr<const FShaderParametersMetadata, ESPMode::ThreadSafe> FOpenColorIOTransformResource::GetShaderParamMetadata() const
{
	return ShaderParametersMetadata;
}

#if WITH_EDITOR
void FOpenColorIOTransformResource::FinishCompilation()
{
	TArray<int32> ShaderMapIdsToFinish;
	GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		for (int32 i = 0; i < ShaderMapIdsToFinish.Num(); i++)
		{
			UE_LOGF(LogShaders, Log, "FinishCompilation()[%d] %ls id %d!", i, *GetFriendlyName(), ShaderMapIdsToFinish[i]);
		}
	
		// Block until the shader maps that we will save have finished being compiled
		GOpenColorIOShaderCompilationManager.FinishCompilation(*GetFriendlyName(), ShaderMapIdsToFinish);

		// Shouldn't have anything left to do...
		TArray<int32> ShaderMapIdsToFinish2;
		GetShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish2);
		ensure(ShaderMapIdsToFinish2.Num() == 0);
	}
}

void FOpenColorIOTransformResource::GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds)
{
	// Build an array of the shader map Id's are not finished compiling.
	if (GameThreadShaderMap && GameThreadShaderMap.IsValid() && !GameThreadShaderMap->IsCompilationFinalized())
	{
		OutShaderMapIds.Add(GameThreadShaderMap->GetCompilingId());
	}
	else if (OutstandingCompileShaderMapIds.Num() != 0)
	{
		OutShaderMapIds.Append(OutstandingCompileShaderMapIds);
	}
}

/**
 * Compiles this color transform for Platform, storing the result in OutShaderMap
 *
 * @param InShaderMapId - the set of static parameters to compile
 * @param InPlatform - the platform to compile for
 * @param OutShaderMap - the shader map to compile
 * @return - true if compile succeeded or was not necessary (shader map for ShaderMapId was found and was complete)
 */
bool FOpenColorIOTransformResource::BeginCompileShaderMap(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FOpenColorIOShaderMap>& OutShaderMap, bool bApplyCompletedShaderMapForRendering, bool bSynchronous)
{
	bool bSuccess = false;

	STAT(double OpenColorIOCompileTime = 0);

	SCOPE_SECONDS_COUNTER(OpenColorIOCompileTime);

	TRefCountPtr<FOpenColorIOShaderMap> NewShaderMap = new FOpenColorIOShaderMap();
	NewShaderMap->AssociateWithAsset(AssetPath);

	// Create a shader compiler environment for the material that will be shared by all jobs from this material
	TRefCountPtr<FSharedShaderCompilerEnvironment> MaterialEnvironment = new FSharedShaderCompilerEnvironment();

	// Compile the shaders for the transform.
	FOpenColorIOCompilationOutput CompilationOutput;
	NewShaderMap->Compile(this, InShaderMapId, MaterialEnvironment, CompilationOutput, InPlatform, bSynchronous, bApplyCompletedShaderMapForRendering);

	if (bSynchronous)
	{
		// If this is a synchronous compile, assign the compile result to the output
		OutShaderMap = NewShaderMap->CompiledSuccessfully() ? NewShaderMap : nullptr;
	}
	else
	{
		UE_LOGF(LogShaders, Log, "BeginCompileShaderMap AddUnique %p %d", this, NewShaderMap->GetCompilingId());
		OutstandingCompileShaderMapIds.AddUnique(NewShaderMap->GetCompilingId());
		
		// Async compile, use nullptr to detect it if used
		OutShaderMap = nullptr;
	}

	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_OpenColorIOShaders, (float)OpenColorIOCompileTime);

	return true;
}

void FOpenColorIOShaderMapId::SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		for (FShaderType* ShaderType : InShaderTypes)
		{
			if (ShaderType != nullptr)
			{
				FShaderTypeDependency Dependency;
				Dependency.ShaderTypeName = ShaderType->GetHashedName();
				Dependency.SourceHash = ShaderType->GetSourceHash(InShaderPlatform);
				ShaderTypeDependencies.Add(Dependency);
			}
		}
	}
}

#endif // WITH_EDITOR

bool FOpenColorIOShaderMapId::ContainsShaderType(const FShaderType* ShaderType) const
{
	for (int32 TypeIndex = 0; TypeIndex < ShaderTypeDependencies.Num(); TypeIndex++)
	{
		if (ShaderTypeDependencies[TypeIndex].ShaderTypeName == ShaderType->GetHashedName())
		{
			return true;
		}
	}

	return false;
}
