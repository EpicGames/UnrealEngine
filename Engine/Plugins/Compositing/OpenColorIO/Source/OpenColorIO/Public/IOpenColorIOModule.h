// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class FOpenColorIODisplayManager;
class FOpenColorIONativeInterchangeConfiguration;

/**
 * Interface for the OpenColorIO module.
 */
class IOpenColorIOModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IOpenColorIOModule& Get()
	{
		static const FName ModuleName = "OpenColorIO";
		return FModuleManager::LoadModuleChecked<IOpenColorIOModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "OpenColorIO";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Returns the collections of ocio display look currently in play
	 */
	virtual FOpenColorIODisplayManager& GetDisplayManager() = 0;
};