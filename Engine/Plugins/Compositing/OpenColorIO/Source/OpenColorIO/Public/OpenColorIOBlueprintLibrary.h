// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "OpenColorIOColorSpace.h"
#include "UObject/ObjectMacros.h"

#include "OpenColorIOBlueprintLibrary.generated.h"


UCLASS(MinimalAPI, meta=(ScriptName="OpenColorIOLibrary"))
class  UOpenColorIOBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 * Applies a rendering pass of the color transform described in the settings
	 *
	 * @param WorldContextObject World from which to get the actual shader feature level we need to render
	 * @param ConversionSettings Settings describing the color space transform to apply
	 * @param InputTexture Texture in the source color space
	 * @param OutputRenderTarget RenderTarget where to draw the input texture in the destination color space
	 * @return True if a rendering command to apply the transform was queued.
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO", meta = (WorldContext = "WorldContextObject"))
	static OPENCOLORIO_API bool ApplyColorSpaceTransform(const UObject* WorldContextObject,	const FOpenColorIOColorConversionSettings& ConversionSettings, UTexture* InputTexture, UTextureRenderTarget2D* OutputRenderTarget);
};
