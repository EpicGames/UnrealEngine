// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenColorIOColorSpace.generated.h"


class UOpenColorIOConfiguration;

/**
 * Structure to identify a ColorSpace as described in an OCIO configuration file. 
 * Members are populated by data coming from a config file.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "OpenColorIO Color Space"))
struct FOpenColorIOColorSpace
{
	GENERATED_BODY()

public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Default constructor. */
	FOpenColorIOColorSpace() = default;
	~FOpenColorIOColorSpace() = default;
	FOpenColorIOColorSpace(const FOpenColorIOColorSpace&) = default;
	FOpenColorIOColorSpace(FOpenColorIOColorSpace&&) = default;
	FOpenColorIOColorSpace& operator=(const FOpenColorIOColorSpace&) = default;
	FOpenColorIOColorSpace& operator=(FOpenColorIOColorSpace&&) = default;
	OPENCOLORIO_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Create and initialize a new instance.
	 */
	FOpenColorIOColorSpace(const FString& InColorSpaceName, int32 InColorSpaceIndex, const FString& InFamilyName, const FString& InDescription = FString());

	/** The ColorSpace name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ColorSpace)
	FString ColorSpaceName;

	/** The index of the ColorSpace in the config */
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "ColorSpaceIndex has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "ColorSpaceIndex has been deprecated."))
	int32 ColorSpaceIndex_DEPRECATED = INDEX_NONE;
#endif

	/** 
	 * The family of this ColorSpace as specified in the configuration file. 
	 * When you have lots of colorspaces, you can regroup them by family to facilitate browsing them. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ColorSpace)
	FString FamilyName;

	/** Colorspace description. */
	UPROPERTY()
	FString Description;

	/** Delimiter used in the OpenColorIO library to make family hierarchies */
	static OPENCOLORIO_API const TCHAR* FamilyDelimiter;

public:
	bool operator==(const FOpenColorIOColorSpace& Other) const { return Other.ColorSpaceName == ColorSpaceName; }
	bool operator!=(const FOpenColorIOColorSpace& Other) const { return !operator==(Other); }

	/**
	 * Get the string representation of this color space.
	 * @return ColorSpace name. 
	 */
	OPENCOLORIO_API FString ToString() const;

	/** Return true if the index and name have been set properly */
	OPENCOLORIO_API bool IsValid() const;

	/** Reset members to default/empty values. */
	OPENCOLORIO_API void Reset();

	/** 
	 * Return the family name at the desired depth level 
	 * @param InDepth Desired depth in the family string. 0 == First layer. 
	 * @return FamilyName at the desired depth. Empty string if depth level doesn't exist.
	 */
	OPENCOLORIO_API FString GetFamilyNameAtDepth(int32 InDepth) const;
};


/**
 * Transformation direction type for display-view transformations.
 */
UENUM(BlueprintType, meta = (DisplayName = "OpenColorIO View Transform Direction"))
enum class EOpenColorIOViewTransformDirection : uint8
{
	Forward = 0     UMETA(DisplayName = "Forward"),
	Inverse = 1     UMETA(DisplayName = "Inverse")
};


USTRUCT(BlueprintType, meta = (DisplayName = "OpenColorIO Display View"))
struct FOpenColorIODisplayView
{
	GENERATED_BODY()

public:
	/** Default constructor. */
	FOpenColorIODisplayView() = default;

	/**
	 * Create and initialize a new instance.
	 */
	OPENCOLORIO_API FOpenColorIODisplayView(FStringView InDisplayName, FStringView InViewName, const FString& InDescription = FString());

	/** Display name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FString Display;

	/** View name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FString View;

	/** Display-view (transform) description. */
	UPROPERTY()
	FString Description;

	OPENCOLORIO_API FString ToString() const;

	/** Return true if the index and name have been set properly */
	OPENCOLORIO_API bool IsValid() const;

	/** Reset members to default/empty values. */
	OPENCOLORIO_API void Reset();

public:
	bool operator==(const FOpenColorIODisplayView& Other) const { return Other.Display == Display && Other.View == View; }
	bool operator!=(const FOpenColorIODisplayView& Other) const { return !operator==(Other); }
};

/**
 * Identifies a OCIO ColorSpace conversion.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "OpenColorIO Color Conversion Settings"))
struct FOpenColorIOColorConversionSettings
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnConversionSettingsChange);

	/** Default constructor. */
	OPENCOLORIO_API FOpenColorIOColorConversionSettings();

	/** The source color space name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	TObjectPtr<UOpenColorIOConfiguration> ConfigurationSource;

	/** The source color space name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FOpenColorIOColorSpace SourceColorSpace;

	/** The destination color space name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FOpenColorIOColorSpace DestinationColorSpace;

	/** The destination display view name. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FOpenColorIODisplayView DestinationDisplayView;

	/** The display view direction. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	EOpenColorIOViewTransformDirection DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;

public:

	OPENCOLORIO_API void PostSerialize(const FArchive& Ar);

	/**
	 * Get a string representation of this conversion.
	 * @return String representation, i.e. "ConfigurationAssetName - SourceColorSpace to DestinationColorSpace".
	 */
	OPENCOLORIO_API FString ToString() const;

	/**
	 * Returns true if the source and destination color spaces are found in the configuration file
	 */
	OPENCOLORIO_API bool IsValid() const;

	/** Returns a string representing the settings' source */
	OPENCOLORIO_API FString GetSourceString() const;

	/** Returns a string representing the settings' destination */
	OPENCOLORIO_API FString GetDestinationString() const;

	/**
	* Reset members to default/empty values.
	* @param InDepth Desired depth in the family string. 0 == First layer. 
	*/
	OPENCOLORIO_API void Reset(bool bResetConfigurationSource = false);

	/**
	* Ensure that the selected source and destination color spaces are valid, resets them otherwise.
	*/
	OPENCOLORIO_API void ValidateColorSpaces();

	bool operator==(const FOpenColorIOColorConversionSettings& Other) const { return Equals(Other); }
	bool operator!=(const FOpenColorIOColorConversionSettings& Other) const { return !Equals(Other); }

	/** Determines if this ColorConversionSettings is the same as another.*/
	bool Equals(const FOpenColorIOColorConversionSettings& Other) const
	{
		return ConfigurationSource    == Other.ConfigurationSource
			&& SourceColorSpace       == Other.SourceColorSpace
			&& DestinationColorSpace  == Other.DestinationColorSpace
			&& DestinationDisplayView == Other.DestinationDisplayView
			&& DisplayViewDirection   == Other.DisplayViewDirection;
	}
	
	/** Whether or not these settings are of the display-view type. */
	OPENCOLORIO_API bool IsDisplayView() const;
};

template<> struct TStructOpsTypeTraits<FOpenColorIOColorConversionSettings> : public TStructOpsTypeTraitsBase2<FOpenColorIOColorConversionSettings>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/**
 * Identifies an OCIO Display look configuration 
 */
USTRUCT(BlueprintType, meta = (DisplayName = "OpenColorIO Display Configuration"))
struct FOpenColorIODisplayConfiguration
{
	GENERATED_BODY()

public:
	/** Whether or not this display configuration is enabled
	 *  Since display look are applied on viewports, this will 
	 * dictate whether it's applied or not to it
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace, DisplayName = "Enable OCIO")
	bool bIsEnabled = false;
	
	/** Conversion to apply when this display is enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ColorSpace)
	FOpenColorIOColorConversionSettings ColorConfiguration;

	OPENCOLORIO_API bool Serialize(FArchive& Ar);
	OPENCOLORIO_API void PostSerialize(const FArchive& Ar);

	/** Determines if this DisplayConfiguration is the same as another.*/
	bool Equals(const FOpenColorIODisplayConfiguration& Other) const
	{
		return bIsEnabled == Other.bIsEnabled && ColorConfiguration.Equals(Other.ColorConfiguration);
	}
};

template<> struct TStructOpsTypeTraits<FOpenColorIODisplayConfiguration> : public TStructOpsTypeTraitsBase2<FOpenColorIODisplayConfiguration>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

