// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CoreMinimal.h"
#include "Engine/VolumeTexture.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIOShared.h"
#include "RenderCommandFence.h"
#include "RHIDefinitions.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "OpenColorIOColorTransform.generated.h"

#define UE_API OPENCOLORIO_API


struct FImageView;
class FOpenColorIOWrapperProcessor;
class UOpenColorIOConfiguration;


/**
 * Object used to generate shader and LUTs from OCIO configuration file and contain required resource to make a color space transform.
 */
UCLASS(MinimalAPI)
class UOpenColorIOColorTransform : public UObject
{
	GENERATED_BODY()

public:

	UE_API UOpenColorIOColorTransform(const FObjectInitializer& ObjectInitializer);
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~UOpenColorIOColorTransform() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:

	/** Initialize resources for color space transform. */
	bool Initialize(const FString& InSourceColorSpace, const FString& InDestinationColorSpace);
	
	/** Initialize resources for display-view transform. */
	UE_API bool Initialize(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection);

	/**
	 * Returns the desired resources required to apply this transform during rendering.
	 */
	UE_API bool GetRenderResources(ERHIFeatureLevel::Type InFeatureLevel, FOpenColorIOTransformResource*& OutShaderResource, TArray<TWeakObjectPtr<UTexture>>& OutTextureResources) const;

	/**
	 * Returns true if shader/texture resources have finished compiling and are ready for use (to be called on the game thread).
	 */
	UE_API bool AreRenderResourcesReady() const;

	/**
	 * Returns true if the current transform corresponds to the specified color spaces.
	 */
	UE_API bool IsTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace) const;

	/**
	 * Returns true if the current transform corresponds to the specified color space and display/view & direction.
	 */
	UE_API bool IsTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection) const;

	/**
	 * Get the transform processor.
	 *
	 * @param OutProcessor Processor wrapper for the current transform.
	 * @return True if the processor was created and is valid.
	 */
	UE_API bool GetTransformProcessor(FOpenColorIOWrapperProcessor& OutProcessor, const TMap<FString, FString>& InContextOverride = {}) const;

	/** Apply the color transform in-place to the specified color. */
	UE_API bool TransformColor(FLinearColor& InOutColor) const;
		
	/** Apply the color transform in-place to the specified image. */
	UE_API bool TransformImage(const FImageView& InOutImage) const;
	
	/** Apply the color transform from the source image to the destination image. (The destination FImageView is const but what it points at is not.) */
	UE_API bool TransformImage(const FImageView& SrcImage, const FImageView& DestImage) const;

	/**
	 * Get the display view direction type, when applicable.
	 *
	 * @param OutDirection The returned direction.
	 * @return True if the transformation is of display-view type.
	 */
	UE_API bool GetDisplayViewDirection(EOpenColorIOViewTransformDirection& OutDirection) const;

	// For all ColorTransforms, UOpenColorIOColorTransform::CacheResourceShadersForRendering
	static UE_API void AllColorTransformsCacheResourceShadersForRendering();

	// Get the owner config's context key-values.
	UE_API TMap<FString, FString> GetContextKeyValues() const;

private:

	/**
	 * Cache resource shaders for rendering.
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 */
	UE_API void CacheResourceShadersForRendering(bool bRegenerateId = false);

	UE_API void CacheShadersForResources(EShaderPlatform InShaderPlatform, FOpenColorIOTransformResource* InResourcesToCache, bool bApplyCompletedShaderMapForRendering, bool bIsCooking, const ITargetPlatform* TargetPlatform = nullptr);

	/**
	 * Helper function to serialize shader maps for the given color transform resources.
	 */
	static UE_API void SerializeOpenColorIOShaderMaps(const TMap<const ITargetPlatform*, TArray<FOpenColorIOTransformResource*>>* PlatformColorTransformResourcesToSavePtr, FArchive& Ar, TArray<FOpenColorIOTransformResource>&  OutLoadedResources);
	
	/**
	 * Helper function to register serialized shader maps for the given color transform resources
	 */
	static UE_API void ProcessSerializedShaderMaps(UOpenColorIOColorTransform* Owner, TArray<FOpenColorIOTransformResource>& LoadedResources, FOpenColorIOTransformResource* (&OutColorTransformResourcesLoaded)[ERHIFeatureLevel::Num]);
	
	/**
	 * Returns a Guid for the LUT based on its unique identifier, name and the OCIO DDC key.
	 */
	static UE_API void GetOpenColorIOLUTKeyGuid(const FString& InProcessorIdentifier, const FName& InName, FGuid& OutLutGuid );

	/**
	 * Helper function returning the color space transform name based on source and destination color spaces.
	 */
	UE_API FString GetTransformFriendlyName() const;

#if WITH_EDITOR
	/**
	 * Fetch shader code and hash from the OCIO library
	 * @return: true if shader could be generated from the library
	 */
	UE_API bool UpdateShaderInfo(FString& OutShaderCodeHash, FString& OutShaderCode, FString& OutRawConfigHash);

	/**
	 * Helper function taking raw 3D LUT data coming from the library and initializing a UVolumeTexture with it.
	 */
	UE_API TObjectPtr<UTexture> CreateTexture3DLUT(const FString& InProcessorIdentifier, const FName& InName, uint32 InLutLength, TextureFilter InFilter, const float* InSourceData);

	/**
	 * Helper function taking raw 1D LUT data coming from the library and initializing a UTexture with it.
	 */
	UE_API TObjectPtr<UTexture> CreateTexture1DLUT(const FString& InProcessorIdentifier, const FName& InName, uint32 InTextureWidth, uint32 InTextureHeight, TextureFilter InFilter, bool bRedChannelOnly, const float* InSourceData );

	/**
	 * Create the transform processor(s) and generate its resources. Editor-only, in game mode GPU resources are already present.
	 */
	UE_API void ProcessTransformForGPU();
#endif

	/* Release all shader maps. */
	UE_API void FlushResourceShaderMaps();

	/* Create the shader parameter metadata, using our built-in struct and any LUT-associated texture and sampler resources. */
	UE_API FShaderParametersMetadata* BuildShaderParamMetadata();

public:

	//~ Begin UObject interface
	UE_API void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void FinishDestroy() override;

#if WITH_EDITOR
	/**
	 * Cache resource shaders for cooking on the given shader platform.
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 * This does not apply completed results to the renderer scenes.
	 * Caller is responsible for deleting OutCachedResource.
	 */
	UE_API void CacheResourceShadersForCooking(EShaderPlatform InShaderPlatform, const ITargetPlatform* TargetPlatform, const FString& InShaderHash, const FString& InShaderCode, const FString& InRawConfigHash, TArray<FOpenColorIOTransformResource*>& OutCachedResources);

	UE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual void ClearCachedCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
	UE_API virtual void ClearAllCachedCookedPlatformData() override;

#endif //WITH_EDITOR
	//~ End UObject interface

	/**
	 * Releases rendering resources used by this color transform.
	 * This should only be called directly if the ColorTransform will not be deleted through the GC system afterward.
	 * FlushRenderingCommands() must have been called before this.
	 */
	UE_API void ReleaseResources();

public:

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	bool bIsDisplayViewType = false;

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	FString SourceColorSpace;

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	FString DestinationColorSpace;

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	FString Display;

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	FString View;

	UPROPERTY(VisibleAnywhere, Category = "ColorSpace")
	EOpenColorIOViewTransformDirection DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;
private:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Map of textures has been deprecated in favor of an array.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Map of textures has been deprecated in favor of an array."))
	TMap<int32, TObjectPtr<UTexture>> Textures_DEPRECATED;
#endif

	/** If the color space transform requires LUTs, this will contains the texture data to do the transform */
	UPROPERTY()
	TArray<TObjectPtr<UTexture>> LookupTextures;

#if WITH_EDITORONLY_DATA
	/** Generated transform shader hash. */
	UPROPERTY()
	FString GeneratedShaderHash;

	/** Generated transform shader function. */
	UPROPERTY()
	FString GeneratedShader;
#endif
	
	/** Inline ColorTransform resources serialized from disk. To be processed on game thread in PostLoad. */
	TArray<FOpenColorIOTransformResource> LoadedTransformResources;
	
	FOpenColorIOTransformResource* ColorTransformResources[ERHIFeatureLevel::Num];

	FRenderCommandFence ReleaseFence;

#if WITH_EDITORONLY_DATA
	/* ColorTransform resources being cached for cooking. */
	TMap<const class ITargetPlatform*, TArray<FOpenColorIOTransformResource*>> CachedColorTransformResourcesForCooking;

	/** Handle so we can unregister the delegate */
	FDelegateHandle FeatureLevelChangedDelegateHandle;
#endif //WITH_EDITORONLY_DATA
};

#undef UE_API
