// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIOConfiguration.generated.h"

#define UE_API OPENCOLORIO_API


class FOpenColorIOWrapperConfig;
class FOpenColorIOTransformResource;
class FTextureResource;
class UOpenColorIOColorTransform;
class UTexture;
struct FImageView;
struct FFileChangeData;
class SNotificationItem;
namespace ERHIFeatureLevel { enum Type : int; }

/**
 * Asset to manage allowed OpenColorIO color spaces. This will create required transform objects.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "OpenColorIO Configuration"))
class UOpenColorIOConfiguration : public UObject
{
	GENERATED_BODY()

public:

	UE_API UOpenColorIOConfiguration(const FObjectInitializer& ObjectInitializer);

public:

	/** Check if the transform resources (shader & optionally lookup textures) are ready for use. */
	UE_API bool IsTransformReady(const FOpenColorIOColorConversionSettings& InSettings);

	/** Get the shader and optionally lookup texture resources to be used by the color transform render pass specified by the settings parameter. */
	UE_API bool GetRenderResources(ERHIFeatureLevel::Type InFeatureLevel, const FOpenColorIOColorConversionSettings& InSettings, FOpenColorIOTransformResource*& OutShaderResource, TArray<TWeakObjectPtr<UTexture>>& OutTextureResources);
	
	/** Check if this configuration contains a transform from the specified source & destination color spaces. */
	UE_API bool HasTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace);
	
	/** Check if this configuration contains a transform from the specified source color space & view destination. */
	UE_API bool HasTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection);

	/** Check if this configuration contains a color space. */
	UE_API bool HasDesiredColorSpace(const FOpenColorIOColorSpace& ColorSpace) const;

	/** Check if this configuration contains a display-view. */
	UE_API bool HasDesiredDisplayView(const FOpenColorIODisplayView& DisplayView) const;

	/** Check if this configuration is valid. (TODO: Rename to avoid confusion with config validation calls provided by the library.) */
	UE_API bool Validate() const;

	/** Apply the color transform in-place to the specified image. */
	UE_API bool TransformColor(const FOpenColorIOColorConversionSettings& InSettings, FLinearColor& InOutColor) const;

	/** Apply the color transform in-place to the specified image. */
	UE_API bool TransformImage(const FOpenColorIOColorConversionSettings& InSettings, const FImageView& InOutImage) const;

	/** Apply the color transform from the source image to the destination image. (The destination FImageView is const but what it points at is not.) */
	UE_API bool TransformImage(const FOpenColorIOColorConversionSettings& InSettings, const FImageView& SrcImage, const FImageView& DestImage) const;

	/** This forces to reload colorspaces and corresponding shaders if those are not loaded already. */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	UE_API void ReloadExistingColorspaces(bool bForce = false);

	/*
	* This method is called by directory watcher when any file or folder is changed in the 
	* directory where raw ocio config is located.
	*/
	UE_API void ConfigPathChangedEvent(const TArray<FFileChangeData>& InFileChanges, const FString InFileMountPath);

	/**
	 * Get the private wrapper implementation of the OCIO config.
	 */
	UE_API FOpenColorIOWrapperConfig* GetConfigWrapper() const;
	
	/**
	 * Get or create the private wrapper implementation of the OCIO config.
	 * Useful for non-editor modes where the config isn't automatically loaded.
	 */
	UE_API FOpenColorIOWrapperConfig* GetOrCreateConfigWrapper();
	
	/** Find the color transform object that corresponds to the specified settings, nullptr if not found. */
	UE_API TObjectPtr<const UOpenColorIOColorTransform> FindTransform(const FOpenColorIOColorConversionSettings& InSettings) const;

protected:

	UE_API void CreateColorTransform(const FString& InSourceColorSpace, const FString& InDestinationColorSpace);
	UE_API void CreateColorTransform(const FString& InSourceColorSpace, const FString& InDisplay, const FString& InView, EOpenColorIOViewTransformDirection InDirection);

	UE_API void CleanupTransforms();

	/** Same as above except user can specify the path manually. */
	UE_API void StartDirectoryWatch(const FString& FilePath);

	/** Stop watching the current directory. */
	UE_API void StopDirectoryWatch();

public:

	//~ Begin UObject interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext RegistryTagsContext) const override;
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

private:

	/** Load the config file to initialize the configuration wrapper object. Automatically called internally. */
	UE_API void LoadConfiguration();

	/** Obtain the hash of the config and additional dependent state such as library version, working color space and context. */
	UE_API bool GetHash(FString& OutHash) const;

#if WITH_EDITOR
	/** This method resets the status of Notification dialog and reacts depending on user's choice. */
	UE_API void OnToastCallback(bool bInReloadColorspaces);
#endif

public:

	/** Get the configuration file path. */
	UFUNCTION(BlueprintGetter)
	const FFilePath& GetConfigurationFile() const { return ConfigurationFile; }

	/** Set the configuration file path. */
	UFUNCTION(BlueprintSetter)
	UE_API void SetConfigurationFile(const FFilePath& InFilePath);

	/** Desired color space tranforms. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transform")
	TArray<FOpenColorIOColorSpace> DesiredColorSpaces;

	/** Desired display view tranforms. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transform", DisplayName="Desired Display-Views")
	TArray<FOpenColorIODisplayView> DesiredDisplayViews;

	/**
	* OCIO context of key-value string pairs, typically used to apply shot-specific looks (such as a CDL color correction, or a 1D grade LUT).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform")
	TMap<FString, FString> Context;

private:
	/** Configuration file path. */
	UPROPERTY(EditAnywhere, BlueprintGetter = GetConfigurationFile, BlueprintSetter = SetConfigurationFile, Category = "Config", meta = (AllowPrivateAccess = true, FilePathFilter = "Config Files (*.ocio, *.ocioz)|*.ocio;*.ocioz", RelativeToGameDir))
	FFilePath ConfigurationFile;

	UPROPERTY()
	TArray<TObjectPtr<UOpenColorIOColorTransform>> ColorTransforms;

private:
	struct FOCIOConfigWatchedDirInfo
	{
		/** A handle to the directory watcher. Gives us the ability to control directory watching status. */
		FDelegateHandle DirectoryWatcherHandle;

		/** Currently watched folder. */
		FString FolderPath;

		/** A handle to Notification message that pops up to notify user of raw config file going out of date. */
		TWeakPtr<SNotificationItem> RawConfigChangedToast;
	};

	/** Information about the currently watched directory. Helps us manage the directory change events. */
	FOCIOConfigWatchedDirInfo WatchedDirectoryInfo;

	/** Private implementation of the OpenColorIO config object. */
	TPimplPtr<FOpenColorIOWrapperConfig> Config = nullptr;

	/** Hash of all of the config content, including relevant external file information. */
	UPROPERTY()
	FString ConfigHash;

	friend class UAssetDefinition_OpenColorIOConfiguration;
	friend class FOpenColorIOConfigurationCustomization;
};

#undef UE_API
