// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"
#include "UObject/GCObject.h"

#define UE_API OPENCOLORIO_API


class FViewportClient;
class FSceneViewFamily;

#define OPENCOLORIO_SCENE_VIEW_EXTENSION_PRIORITY 100

/** 
 * View extension applying an OCIO Display Look to the viewport we're attached to
 */
class FOpenColorIODisplayExtension : public FSceneViewExtensionBase, public FGCObject
{
public:

	UE_API FOpenColorIODisplayExtension(const FAutoRegister& AutoRegister, FViewportClient* AssociatedViewportClient = nullptr);
	
	//~ Begin ISceneViewExtension interface	
	virtual int32 GetPriority() const override { return OPENCOLORIO_SCENE_VIEW_EXTENSION_PRIORITY; }
	UE_API virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;

	UE_API virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;
	UE_API virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	//~End ISceneVIewExtension interface

	UE_API FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs);

	//~Begin FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FOpenColorIODisplayExtension");
	}
	//~End FGCObject interface

	void SetDisplayConfiguration(const FOpenColorIODisplayConfiguration& InDisplayConfiguration) { DisplayConfiguration = InDisplayConfiguration; };
public:
	/** Returns the ViewportClient this extension is currently attached to */
	FViewportClient* GetAssociatedViewportClient() { return LinkedViewportClient; }

	/** Returns the current display configuration in a const manner */
	const FOpenColorIODisplayConfiguration& GetDisplayConfiguration() const { return DisplayConfiguration; }
	
	/** Returns the current display configuration to be updated */
	FOpenColorIODisplayConfiguration& GetDisplayConfiguration() { return DisplayConfiguration; }

private:

	/** Cached pass resources required to apply conversion for render thread */
	FOpenColorIORenderPassResources CachedResourcesRenderThread;

	/** Configuration to apply during post render callback */
	FOpenColorIODisplayConfiguration DisplayConfiguration;

	/** ViewportClient to which we are attached */
	FViewportClient* LinkedViewportClient = nullptr;

};


#undef UE_API
