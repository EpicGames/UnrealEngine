// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenColorIODisplayExtension.h"

#include "OpenColorIODisplayExtensionWrapper.generated.h"

#define UE_API OPENCOLORIO_API

/** 
 * This Blueprintable object can hold an OCIO Scene View Extension. 
 * You can change its OCIO config, and specify the context in which you want it to be active on.
 */
UCLASS(MinimalAPI, Blueprintable, meta = (DisplayName = "OpenColorIO Display Extension Wrapper"))
class UOpenColorIODisplayExtensionWrapper : public UObject
{
	GENERATED_BODY()

public:

	// Gets the display extension OCIO configuration.
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get OpenColorIO Configuration"), Category = "OpenColorIO")
	UE_API FOpenColorIODisplayConfiguration GetOpenColorIOConfiguration() const;

	// Sets the display extension OCIO configuration.
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set OpenColorIO Configuration"), Category = "OpenColorIO")
	UE_API void SetOpenColorIOConfiguration(FOpenColorIODisplayConfiguration InDisplayConfiguration);

	// Sets a single activation function. Will remove any others.
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	UE_API void SetSceneExtensionIsActiveFunction(const FSceneViewExtensionIsActiveFunctor& IsActiveFunction);

	// Sets an array of activation functions. Will remove any others.
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	UE_API void SetSceneExtensionIsActiveFunctions(const TArray<FSceneViewExtensionIsActiveFunctor>& IsActiveFunctions);

	// Removes the extension.
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	UE_API void RemoveSceneExtension();

	// Creates an instance of this object, configured with the given arguments (OCIO and activation function).
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Create OpenColorIO Display Extension"), Category = "OpenColorIO")
	static UE_API UOpenColorIODisplayExtensionWrapper* CreateOpenColorIODisplayExtension(
		FOpenColorIODisplayConfiguration InDisplayConfiguration, 
		const FSceneViewExtensionIsActiveFunctor& IsActiveFunction);

	// Creates an instance of this object, configured for use in game with the given OCIO configuration.
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Create In-Game OpenColorIO Display Extension"), Category = "OpenColorIO")
	static UE_API UOpenColorIODisplayExtensionWrapper* CreateInGameOpenColorIODisplayExtension(
		FOpenColorIODisplayConfiguration InDisplayConfiguration);
private:

	// Creates OCIO Extension if it doesn't exist. 
	UE_API void CreateDisplayExtensionIfNotExists();

private:

	// Holds the OCIO Extension.
	TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe> DisplayExtension;
};


#undef UE_API
