// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"

#define UE_API OPENCOLORIO_API

class FOpenColorIODisplayExtension;
class FViewportClient;

class FOpenColorIODisplayManager : public TSharedFromThis<FOpenColorIODisplayManager>
{
public:

	/** Will return the configuration associated to the desired viewport or create one if it's not tracked */
	UE_API FOpenColorIODisplayConfiguration& FindOrAddDisplayConfiguration(FViewportClient* InViewportClient);

	/** Returns the configuration for a given viewport if it was found, nullptr otherwise */
	UE_API const FOpenColorIODisplayConfiguration* GetDisplayConfiguration(const FViewportClient* InViewportClient) const;

	/** Remove display configuration associated with this viewport */
	UE_API bool RemoveDisplayConfiguration(const FViewportClient* InViewportClient);

	/** Whether or not InViewport has a display configuration linked to it */
	UE_API bool IsTrackingViewport(const FViewportClient* InViewportClient) const;

protected:

	/** List of DisplayExtension created when a viewport is asked to be tracked. It contains the configuration. */
	TArray<TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe>> DisplayExtensions;
};

#undef UE_API
