// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "RHIFwd.h"

#define UE_API OPENCOLORIO_API

struct FOpenColorIOColorConversionSettings;
class FOpenColorIOTransformResource;
class FRDGBuilder;
struct FScreenPassRenderTarget;
struct FScreenPassTexture;
struct FScreenPassViewInfo;
class FTextureResource;
class FSceneView;
class FSceneViewFamily;
class UTexture;
class UTextureRenderTarget2D;
class UWorld;

/** Resources needed by the FOpenColorIORendering pass function. */
struct FOpenColorIORenderPassResources
{
	/** Color transform pass (generated) shader. */
	FOpenColorIOTransformResource* ShaderResource = nullptr;
	
	/** Collection of LUT textures needed by the shader. */
	TArray<TWeakObjectPtr<UTexture>> TextureResources = {};

	/** Color transform string description. */
	FString TransformName = FString();

	/** Resources are valid when the shader resource is not null. */
	bool IsValid() const { return ShaderResource != nullptr; }
};

/** Option to transform the render target alpha before applying the color transform. */
enum class EOpenColorIOTransformAlpha : uint32
{
	None = 0,
	Unpremultiply = 1,
	InvertUnpremultiply = 2
};

/** Entry point to trigger OpenColorIO conversion rendering */
class FOpenColorIORendering
{
public:
	FOpenColorIORendering() = delete;

	/**
	 * Applies the color transform described in the settings
	 *
	 * @param InWorld World from which to get the actual shader feature level we need to render
	 * @param InSettings Settings describing the color space transform to apply
	 * @param InTexture Texture in the source color space
	 * @param OutRenderTarget RenderTarget where to draw the input texture in the destination color space
	 * @return True if a rendering command to apply the transform was queued.
	 */
	static UE_API bool ApplyColorTransform(UWorld* InWorld, const FOpenColorIOColorConversionSettings& InSettings, UTexture* InTexture, UTextureRenderTarget2D* OutRenderTarget);


	/**
	 * Get render pass resources on the game thread, to be enqueue-copied for later use by the `AddPass_RenderThread` function.
	 *
	 * @param InSettings Settings describing the color space transform to apply
	 * @param InFeatureLevel Rendering feature level
	 * @return FOpenColorIORenderPassResources Render pass resources to be enqueue-copied onto the render thread.
	 */
	static UE_API FOpenColorIORenderPassResources GetRenderPassResources(const FOpenColorIOColorConversionSettings& InSettings, ERHIFeatureLevel::Type InFeatureLevel);

	/**
	 * Applies the color transform RDG pass with the provided resources.
	 *
	 * @param GraphBuilder Render graph builder
	 * @param ViewInfo Scene view with additional information
	 * @param FeatureLevel - Shader model.
	 * @param Input Input color texture
	 * @param Output Destination render target
	 * @param InPassInfo OpenColorIO shader and texture resources
	 * @param InGamma Display gamma
	 * @param TransformAlpha Whether to unpremult/invert before applying the color transform
	 */
	static UE_API void AddPass_RenderThread(
		FRDGBuilder& GraphBuilder,
		FScreenPassViewInfo ViewInfo,
		ERHIFeatureLevel::Type FeatureLevel,
		const FScreenPassTexture& Input,
		const FScreenPassRenderTarget& Output,
		const FOpenColorIORenderPassResources& InPassInfo,
		float InGamma,
		EOpenColorIOTransformAlpha TransformAlpha = EOpenColorIOTransformAlpha::None);

	/**
	 * Similar to the above, except gamma and feature level are handled by the function itself based on provided View.
	 *
	 * @param GraphBuilder Render graph builder
	 * @param View Scene view with additional information
	 * @param Input Input color texture
	 * @param Output Destination render target
	 * @param InPassInfo OpenColorIO shader and texture resources
	 * @param TransformAlpha Whether to unpremult/invert before applying the color transform
	 */
	static UE_API void AddPass_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FScreenPassTexture& Input,
		const FScreenPassRenderTarget& Output,
		const FOpenColorIORenderPassResources& InPassInfo,
		EOpenColorIOTransformAlpha TransformAlpha = EOpenColorIOTransformAlpha::None);

	/** 
	* This function sets up provided view and related family to properly support OCIO. 
	*/
	static UE_API void PrepareView(FSceneViewFamily& InViewFamily, FSceneView& InView);
};

#undef UE_API
