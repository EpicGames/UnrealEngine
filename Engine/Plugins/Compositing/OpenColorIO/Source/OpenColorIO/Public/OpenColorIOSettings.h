// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"

#include "OpenColorIOSettings.generated.h"

#define UE_API OPENCOLORIO_API

/**
 * Rendering settings.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UOpenColorIOSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UOpenColorIOSettings();

	//~ Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface

public:

	UPROPERTY(config, EditAnywhere, Category = Transform, meta = (
		DisplayName = "Enable Legacy Gpu Processor",
		ToolTip = "Whether to enable OCIO V1's legacy gpu processor.",
		ConfigRestartRequired = true))
	uint8 bUseLegacyProcessor : 1;

	UPROPERTY(config, EditAnywhere, Category = Transform, meta = (
		DisplayName = "32-bit float LUTs",
		ToolTip = "Whether to create lookup table texture resources in 32-bit float format (higher performance requirements).",
		ConfigRestartRequired = true))
	uint8 bUse32fLUT : 1;

	UPROPERTY(config, EditAnywhere, Category = Transform, meta = (
		DisplayName = "Support inverse view transforms",
		ToolTip = "Whether inverse view transforms are cached and supported. Disabled by default, to minimize the number of transform combinations.",
		ConfigRestartRequired = true))
	uint8 bSupportInverseViewTransforms : 1;
};

#undef UE_API
