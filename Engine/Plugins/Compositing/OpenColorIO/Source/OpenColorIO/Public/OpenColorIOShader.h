// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenColorIOShader.h: Shader base classes
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "OpenColorIOShaderType.h"
#include "OpenColorIOShared.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"

class FTextureResource;
class FRHITexture;

BEGIN_SHADER_PARAMETER_STRUCT(FOpenColorIOPixelShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputTextureSampler)
	SHADER_PARAMETER(float, Gamma)
	SHADER_PARAMETER(uint32, TransformAlpha)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FOpenColorIOPixelShader : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOpenColorIOPixelShader, OpenColorIO, OPENCOLORIO_API);

public:
	FOpenColorIOPixelShader() = default;
	FOpenColorIOPixelShader(
		const FOpenColorIOShaderType::CompiledShaderInitializerType& Initializer
	);

	MS_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT) struct FParameters
	{
	};
};

BEGIN_SHADER_PARAMETER_STRUCT(FOpenColorIOInvalidShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FOpenColorIOInvalidPixelShader : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FOpenColorIOInvalidPixelShader, OPENCOLORIO_API);
	SHADER_USE_PARAMETER_STRUCT(FOpenColorIOInvalidPixelShader, FGlobalShader);

	using FParameters = FOpenColorIOInvalidShaderParameters;
};

FRHITexture* OpenColorIOGetMiniFontTexture();
