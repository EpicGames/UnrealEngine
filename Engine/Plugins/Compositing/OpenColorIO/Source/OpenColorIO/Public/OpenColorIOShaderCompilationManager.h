// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "ShaderCompiler.h"

class FShaderType;


/** Information tracked for each shader compile worker process instance. */
struct FOpenColorIOShaderCompileWorkerInfo
{
	/** Process handle of the worker app once launched.  Invalid handle means no process. */
	FProcHandle WorkerProcess;

	/** Tracks whether tasks have been issued to the worker. */
	bool bIssuedTasksToWorker;

	/** Whether the worker has been launched for this set of tasks. */
	bool bLaunchedWorker;

	/** Tracks whether all tasks issued to the worker have been received. */
	bool bComplete;

	/** Time at which the worker started the most recent batch of tasks. */
	double StartTime;

	/** Jobs that this worker is responsible for compiling. */
	TArray<FShaderCommonCompileJobPtr> QueuedJobs;

	FOpenColorIOShaderCompileWorkerInfo() :
		bIssuedTasksToWorker(false),
		bLaunchedWorker(false),
		bComplete(false),
		StartTime(0)
	{
	}

	// warning: not virtual
	~FOpenColorIOShaderCompileWorkerInfo()
	{
		if (WorkerProcess.IsValid())
		{
			FPlatformProcess::TerminateProc(WorkerProcess);
			FPlatformProcess::CloseProc(WorkerProcess);
		}
	}
};


/** Results for a single compiled shader map. */
struct FOpenColorIOShaderMapCompileResults
{
	FOpenColorIOShaderMapCompileResults() :
		NumJobsQueued(0),
		bAllJobsSucceeded(true)
	{}

	int32 NumJobsQueued;
	bool bAllJobsSucceeded;
	TArray<FShaderCommonCompileJobPtr> FinishedJobs;
};


/** Results for a single compiled and finalized shader map. */
struct FOpenColorIOShaderMapFinalizeResults : public FOpenColorIOShaderMapCompileResults
{
	/** Tracks finalization progress on this shader map. */
	int32 FinalizeJobIndex;

	FOpenColorIOShaderMapFinalizeResults(const FOpenColorIOShaderMapCompileResults& InCompileResults) :
		FOpenColorIOShaderMapCompileResults(InCompileResults),
		FinalizeJobIndex(0)
	{}
};


// handles finished shader compile jobs, applying of the shaders to their config asset, and some error handling
//
class FOpenColorIOShaderCompilationManager
{
public:
	FOpenColorIOShaderCompilationManager();
	~FOpenColorIOShaderCompilationManager();

	OPENCOLORIO_API void Tick(float DeltaSeconds = 0.0f);
	OPENCOLORIO_API void AddJobs(TArray<FShaderCommonCompileJobPtr> InNewJobs);
	OPENCOLORIO_API void ProcessAsyncResults();

	void FinishCompilation(const TCHAR* InTransformName, const TArray<int32>& ShaderMapIdsToFinishCompiling);

private:
	void ProcessCompiledOpenColorIOShaderMaps(TMap<int32, FOpenColorIOShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	void InitWorkerInfo();

	TArray<FShaderCommonCompileJobPtr> JobQueue;

	/** Map from shader map Id to the compile results for that map, used to gather compiled results. */
	TMap<int32, FOpenColorIOShaderMapCompileResults> OpenColorIOShaderMapJobs;

	/** Map from shader map id to results being finalized.  Used to track shader finalizations over multiple frames. */
	TMap<int32, FOpenColorIOShaderMapFinalizeResults> PendingFinalizeOpenColorIOShaderMaps;

	TArray<struct FOpenColorIOShaderCompileWorkerInfo*> WorkerInfos;
};

extern OPENCOLORIO_API FOpenColorIOShaderCompilationManager GOpenColorIOShaderCompilationManager;

#endif // WITH_EDITOR
