// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenColorIOShaderType.h: OpenColorIO shader type definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GlobalShader.h"
#include "Shader.h"

/** A macro to implement OpenColorIO Color Space Transform shaders. */
#define IMPLEMENT_OCIO_SHADER_TYPE(TemplatePrefix,ShaderClass,SourceFilename,FunctionName,Frequency) \
	IMPLEMENT_SHADER_TYPE( \
		TemplatePrefix, \
		ShaderClass, \
		SourceFilename, \
		FunctionName, \
		Frequency \
		);

class FOpenColorIOTransformResource;
class FShaderCompileJob;
class FShaderCommonCompileJob;
class FUniformExpressionSet;


/** Called for every OpenColorIO shader to update the appropriate stats. */
extern void UpdateOpenColorIOShaderCompilingStats(const FOpenColorIOTransformResource* InShader);

struct FOpenColorIOShaderPermutationParameters : public FGlobalShaderPermutationParameters
{
	const FOpenColorIOTransformResource* Transform;

	FOpenColorIOShaderPermutationParameters(FName InGlobalShaderName, EShaderPlatform InPlatform, const FOpenColorIOTransformResource* InTransform)
		: FGlobalShaderPermutationParameters(InGlobalShaderName, InPlatform)
		, Transform(InTransform)
	{}
};

/**
 * A shader meta type for OpenColorIO-linked shaders.
 */
class FOpenColorIOShaderType : public FShaderType
{
public:
	struct FParameters : public FShaderType::FParameters
	{
		TSharedRef<const FShaderParametersMetadata> ShaderParamMetadata;

		FParameters(const TSharedRef<const FShaderParametersMetadata>& InShaderParamMetadata)
			: ShaderParamMetadata(InShaderParamMetadata)
		{
		}
	};

	struct CompiledShaderInitializerType : FShaderCompiledShaderInitializerType
	{
		const FString DebugDescription;

		CompiledShaderInitializerType(
			const FShaderType* InType,
			const FParameters* InParameters,
			int32 InPermutationId,
			const FShaderCompilerOutput& CompilerOutput,
			uint64 InOCIOShaderMapHash,
			const FString& InDebugDescription
			)
		: FShaderCompiledShaderInitializerType(InType, InParameters, InPermutationId, CompilerOutput, InOCIOShaderMapHash, nullptr, nullptr)
		, DebugDescription(InDebugDescription)
		{}
	};

	FOpenColorIOShaderType(
		FTypeLayoutDesc& InTypeLayout,
		const TCHAR* InName,
		const TCHAR* InSourceFilename,
		const TCHAR* InFunctionName,
		uint32 InFrequency,					// ugly - ignored for OCIO shaders but needed for IMPLEMENT_SHADER_TYPE macro magic
		int32 InTotalPermutationCount,
		int32 InTotalSpecializationCount,
		ConstructSerializedType InConstructSerializedRef,
		ConstructCompiledType InConstructCompiledRef,
		ShouldCompilePermutationType InShouldCompilePermutationRef,
		ShouldPrecachePermutationType InShouldPrecachePermutationRef,
		GetUnspecializedIdType InGetUnspecializedIdRef,
		GetSpecializationValuesType InGetSpecializationValuesRef,
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef,
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
		ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef,
		ValidateCompiledResultType InValidateCompiledResultRef,
		GetOverrideJobPriorityType InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
		uint32 InTypeSize,
		const FShaderParametersMetadata* InRootParametersMetadata = nullptr
#if WITH_EDITOR
		, GetPermutationIdStringType InGetPermutationIdStringRef = nullptr
#endif // WITH_EDITOR
	):
		FShaderType(EShaderTypeForDynamicCast::OCIO, InTypeLayout, InName, InSourceFilename, InFunctionName, SF_Pixel, InTotalPermutationCount, InTotalSpecializationCount,

			InConstructSerializedRef,
			InConstructCompiledRef,
			InShouldCompilePermutationRef,
			InShouldPrecachePermutationRef,
			InGetUnspecializedIdRef,
			InGetSpecializationValuesRef,
			InGetRayTracingPayloadTypeRef,
			InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
			InModifyCompilationEnvironmentRef,
			InValidateCompiledResultRef,
			InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
			InTypeSize,
			InRootParametersMetadata
#if WITH_EDITOR
			, InGetPermutationIdStringRef
#endif // WITH_EDITOR
		)
	{
		check(InTotalPermutationCount == 1);
	}

#if WITH_EDITOR
	/**
	 * Enqueues a compilation for a new shader of this type.
	 * @param InColorTransform - The ColorTransform to link the shader with.
	 */
	void BeginCompileShader(
			uint32 ShaderMapId,
			const FOpenColorIOTransformResource* InColorTransform,
			FSharedShaderCompilerEnvironment* CompilationEnvironment,
			EShaderPlatform Platform,
			TArray<TRefCountPtr<class FShaderCommonCompileJob>>& NewJobs,
			FShaderTarget Target
		);

	/**
	 * Either creates a new instance of this type or returns an equivalent existing shader.
	 * @param InOCIOShaderMapHash - The hash value of the shadermap which will reference this shader
	 * @param CurrentJob - Compile job that was enqueued by BeginCompileShader.
	 * @param InDebugDescription - Description of the shader job (type, permutation ID, etc.)
	 * @return The constructed FShader instance for the completed job
	 */
	FShader* FinishCompileShader(
		uint64 InOCIOShaderMapHash,
		const FShaderCompileJob& CurrentJob,
		const FString& InDebugDescription
		) const;
#endif // WITH_EDITOR

	/**
	 * Checks if the shader type should be cached for a particular platform and color transform.
	 * @param Platform - The platform to check.
	 * @param InColorTransform - The color transform to check.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCache(EShaderPlatform InPlatform, const FOpenColorIOTransformResource* InColorTransform) const
	{
		return ShouldCompilePermutation(FOpenColorIOShaderPermutationParameters(GetFName(), InPlatform, InColorTransform));
	}

#if WITH_EDITOR
protected:
	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param InPlatform - Platform to compile for.
	 * @param OutEnvironment - The shader compile environment that the function modifies.
	 */
	void SetupCompileEnvironment(EShaderPlatform InPlatform, const FOpenColorIOTransformResource* InColorTransform, FShaderCompilerEnvironment& OutEnvironment) const;
#endif // WITH_EDITOR
};
