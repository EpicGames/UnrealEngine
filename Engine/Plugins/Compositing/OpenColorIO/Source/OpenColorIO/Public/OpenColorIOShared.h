// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenColorIOShared.h: Shared OpenColorIO definitions.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"
#include "RenderingThread.h"
#include "RHIFeatureLevel.h"
#include "SceneTypes.h"
#include "Shader.h"
#include "StaticParameterSet.h"
#include "Templates/RefCounting.h"

/** Enum used to indicate whether the working color space should be used as a source or destination. */
enum class EOpenColorIOWorkingColorSpaceTransform : uint8
{
	None = 0,
	Source = 1,
	Destination = 2
};

class FOpenColorIOTransformResource;
class FOpenColorIOShaderMap;
class FOpenColorIOPixelShader;
class FOpenColorIOShaderMapId;


/** Stores outputs from the color transform compile that need to be saved. */
class FOpenColorIOCompilationOutput
{
	DECLARE_TYPE_LAYOUT(FOpenColorIOCompilationOutput, NonVirtual);
public:
	FOpenColorIOCompilationOutput()
	{}
};

/** Contains all the information needed to uniquely identify a FOpenColorIOShaderMapID. */
class FOpenColorIOShaderMapId
{
	DECLARE_TYPE_LAYOUT(FOpenColorIOShaderMapId, NonVirtual);
public:
	/** Feature level that the shader map is going to be compiled for.  */
	LAYOUT_FIELD(ERHIFeatureLevel::Type, FeatureLevel);

	/**
	 * The hash computed from the shader code has and config hash.
	 */
	LAYOUT_FIELD(FMemoryImageString, ShaderCodeAndConfigHash);
	
	/** Shader types of shaders that are inlined in this shader map in the DDC. */
	LAYOUT_FIELD(TMemoryImageArray<FShaderTypeDependency>, ShaderTypeDependencies);

	/*
	* Type layout parameters of the memory image
	*/
	LAYOUT_FIELD(FPlatformTypeLayoutParameters, LayoutParams);

public:
	FOpenColorIOShaderMapId()
		: FeatureLevel(ERHIFeatureLevel::SM5)
	{ }

	~FOpenColorIOShaderMapId() = default;

#if WITH_EDITOR
	void SetShaderDependencies(const TArray<FShaderType*>& InShaderTypes, EShaderPlatform InShaderPlatform);
#endif // WITH_EDITOR

	//void Serialize(FArchive& Ar);

	friend uint32 GetTypeHash(const FOpenColorIOShaderMapId& Ref)
	{
		return GetTypeHash(Ref.ShaderCodeAndConfigHash);
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this) + ShaderTypeDependencies.GetAllocatedSize();
	}

	/** Hashes the color transform-specific part of this shader map Id. */
	uint64 GetOpenColorIOHash() const;

	/**
	 * Tests this set against another for equality, disregarding override settings.
	 *
	 * @param InReferenceSet	The set to compare against
	 * @return					true if the sets are equal
	 */
	bool operator==(const FOpenColorIOShaderMapId& InReferenceSet) const;

	bool operator!=(const FOpenColorIOShaderMapId& InReferenceSet) const
	{
		return !(*this == InReferenceSet);
	}

#if WITH_EDITOR
	/** Appends string representations of this Id to a key string. */
	void AppendKeyString(FString& OutKeyString) const;
#endif // WITH_EDITOR

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	bool ContainsShaderType(const FShaderType* ShaderType) const;
};

class FOpenColorIOShaderMapContent : public FShaderMapContent
{
	using Super = FShaderMapContent;
	friend class FOpenColorIOShaderMap;
	DECLARE_TYPE_LAYOUT(FOpenColorIOShaderMapContent, NonVirtual);
private:
	explicit FOpenColorIOShaderMapContent(EShaderPlatform InPlatform) : Super(InPlatform) {}

	/** The ColorTransform's user friendly name, typically the source and destination color spaces. */
	LAYOUT_FIELD(FMemoryImageString, FriendlyName);

	/** The static parameter set that this shader map was compiled with */
	LAYOUT_FIELD(FOpenColorIOShaderMapId, ShaderMapId);

	/** Shader compilation output */
	LAYOUT_FIELD(FOpenColorIOCompilationOutput, OpenColorIOCompilationOutput);
};


/**
 * The shaders required to apply an OCIO color transform.
 */
class FOpenColorIOShaderMap : public TShaderMap<FOpenColorIOShaderMapContent, FShaderMapPointerTable>, public FDeferredCleanupInterface
{
public:
	using Super = TShaderMap<FOpenColorIOShaderMapContent, FShaderMapPointerTable>;

	/**
	 * Finds the shader map for a color transform.
	 * @param InShaderMapId - Id to find
	 * @param Platform - The platform to lookup for
	 * @return nullptr if no cached shader map was found.
	 */
	static FOpenColorIOShaderMap* FindId(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform);

	// ShaderMap interface
	template<typename ShaderType> TShaderRef<ShaderType> GetShader() const { return TShaderRef<ShaderType>(GetContent()->GetShader<ShaderType>(), *this); }
	TShaderRef<FShader> GetShader(FShaderType* ShaderType) const { return TShaderRef<FShader>(GetContent()->GetShader(ShaderType), *this); }

#if WITH_EDITOR
	/**
	 * Attempts to load the shader map for the given color transform from the Derived Data Cache.
	 * If InOutShaderMap is valid, attempts to load the individual missing shaders instead.
	 */
	static void LoadFromDerivedDataCache(const FOpenColorIOTransformResource* InColorTransform, const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform, TRefCountPtr<FOpenColorIOShaderMap>& InOutShaderMap);
#endif // WITH_EDITOR

	FOpenColorIOShaderMap();

	// Destructor.
	~FOpenColorIOShaderMap();

#if WITH_EDITOR
	/**
	 * Compiles the shaders for a color transform and caches them in this shader map.
	 * @param InColorTransform - The color transform to compile shaders for.
	 * @param InShaderMapId - the set of static parameters to compile for
	 * @param InPlatform - The platform to compile to
	 */
	OPENCOLORIO_API void Compile(
		FOpenColorIOTransformResource* InColorTransform,
		const FOpenColorIOShaderMapId& InShaderMapId,
		TRefCountPtr<FSharedShaderCompilerEnvironment> InCompilationEnvironment,
		const FOpenColorIOCompilationOutput& InOpenColorIOCompilationOutput,
		EShaderPlatform InPlatform,
		bool bSynchronousCompile,
		bool bApplyCompletedShaderMapForRendering
		);

	/** Sorts the incoming compiled jobs into the appropriate OCIO shader maps, and finalizes this shader map so that it can be used for rendering. */
	bool ProcessCompilationResults(const TArray<TRefCountPtr<class FShaderCommonCompileJob>>& InCompilationResults, int32& InOutResultIndex, float& InOutTimeBudget);
#endif // WITH_EDITOR

	/**
	 * Checks whether the shader map is missing any shader types necessary for the given color transform.
	 * @param InColorTransform - The OpenColorIO ColorTransform which is checked.
	 * @return True if the shader map has all of the shader types necessary.
	 */
	bool IsComplete(const FOpenColorIOTransformResource* InColorTransform, bool bSilent);

#if WITH_EDITOR
	/**
	 * Checks to see if the shader map is already being compiled for another color transform, and if so
	 * adds the specified color transform to the list to be applied to once the compile finishes.
	 * @param InColorTransform - The OpenColorIO ColorTransform we also wish to apply the compiled shader map to.
	 * @return True if the shader map was being compiled and we added ColorTransform to the list to be applied.
	 */
	bool TryToAddToExistingCompilationTask(FOpenColorIOTransformResource* InColorTransform);
#endif // WITH_EDITOR

	/** Builds a list of the shaders in a shader map. */
	OPENCOLORIO_API void GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;
	OPENCOLORIO_API virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	OPENCOLORIO_API virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;

	/** Registers a OpenColorIO shader map in the global map so it can be used by OpenColorIO ColorTransform. */
	void Register(EShaderPlatform InShaderPlatform);
	// Reference counting.
	OPENCOLORIO_API  void AddRef();
	OPENCOLORIO_API  void Release();

#if WITH_EDITOR
	/** Removes a ColorTransform from OpenColorIOShaderMapsBeingCompiled. */
	OPENCOLORIO_API static void RemovePendingColorTransform(FOpenColorIOTransformResource* InColorTransform);
#endif // WITH_EDITOR

	/** Serializes the shader map. */
	bool Serialize(FArchive& Ar);

#if WITH_EDITOR
	/** Saves this shader map to the derived data cache. */
	void SaveToDerivedDataCache();
#endif // WITH_EDITOR

	// Accessors.
	const FOpenColorIOShaderMapId& GetShaderMapId() const	{ return GetContent()->ShaderMapId; }
	EShaderPlatform GetShaderPlatform() const				{ return GetContent()->GetShaderPlatform(); }
	const FMemoryImageString& GetFriendlyName() const		{ return GetContent()->FriendlyName; }
	uint32 GetCompilingId() const							{ return CompilingId; }
	bool IsCompilationFinalized() const						{ return bCompilationFinalized; }
	bool CompiledSuccessfully() const						{ return bCompiledSuccessfully; }

	bool IsValid() const
	{
		return bCompilationFinalized && bCompiledSuccessfully && !bDeletedThroughDeferredCleanup;
	}

	int32 GetNumRefs() const { return NumRefs; }
	uint32 GetCompilingId()  { return CompilingId; }

#if WITH_EDITOR
	static TMap<TRefCountPtr<FOpenColorIOShaderMap>, TArray<FOpenColorIOTransformResource*> > &GetInFlightShaderMaps() { return OpenColorIOShaderMapsBeingCompiled; }

	void SetCompiledSuccessfully(bool bSuccess) { bCompiledSuccessfully = bSuccess; }
#endif // WITH_EDITOR
private:

	/**
	 * A global map from a ColorTransform's ID and static switch set to any shader map cached for that color transform.
	 * Note: this does not necessarily contain all color transform shader maps in memory.  Shader maps with the same key can evict each other.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TMap<FOpenColorIOShaderMapId, FOpenColorIOShaderMap*> GIdToOpenColorIOShaderMap[SP_NumPlatforms];

	/**
	 * All color transform shader maps in memory.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TArray<FOpenColorIOShaderMap*> AllOpenColorIOShaderMaps;

#if WITH_EDITOR
	/** Next value for CompilingId. */
	static uint32 NextCompilingId;

	/** Tracks resources and their shader maps that need to be compiled but whose compilation is being deferred. */
	static TMap<TRefCountPtr<FOpenColorIOShaderMap>, TArray<FOpenColorIOTransformResource*> > OpenColorIOShaderMapsBeingCompiled;
#endif // WITH_EDITOR

	/** Uniquely identifies this shader map during compilation, needed for deferred compilation where shaders from multiple shader maps are compiled together. */
	uint32 CompilingId;

	mutable int32 NumRefs;

	/** Used to catch errors where the shader map is deleted directly. */
	bool bDeletedThroughDeferredCleanup;

	/** Indicates whether this shader map has been registered in GIdToOpenColorsIOShaderMap */
	uint32 bRegistered : 1;

	/**
	 * Indicates whether this shader map has had ProcessCompilationResults called after Compile.
	 * The shader map must not be used on the rendering thread unless bCompilationFinalized is true.
	 */
	uint32 bCompilationFinalized : 1;

	uint32 bCompiledSuccessfully : 1;

	/** Indicates whether the shader map should be stored in the shader cache. */
	uint32 bIsPersistent : 1;

	uint32 bHasFrozenContent : 1;

#if WITH_EDITOR
	FShader* ProcessCompilationResultsForSingleJob(const TRefCountPtr<class FShaderCommonCompileJob>& SingleJob, uint64 InShaderMapHash);
#endif

	bool IsOpenColorIOShaderComplete(const FOpenColorIOTransformResource* InColorTransform, const FOpenColorIOShaderType* InShaderType, bool bSilent);

	friend class FShaderCompilingManager;
};


/**
 * FOpenColorIOTransformResource represents a OpenColorIO color transform to the shader compilation process
 */
class FOpenColorIOTransformResource
{
public:

	/**
	 * Minimal initialization constructor.
	 */
	OPENCOLORIO_API FOpenColorIOTransformResource();

	/**
	 * Destructor
	 */
	OPENCOLORIO_API virtual ~FOpenColorIOTransformResource();

	/**
	 * Caches the shaders for this color transform with no static parameters on the given platform.
	 * This is used by UOpenColorIOColorTransform
	 */
	OPENCOLORIO_API bool CacheShaders(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);
	OPENCOLORIO_API bool CacheShaders(const FOpenColorIOShaderMapId& InShaderMapId, EShaderPlatform InPlatform, bool bApplyCompletedShaderMapForRendering, bool bSynchronous);

	/** Returns the shader parameter metadata for the generated shader (built-in FOpenColorIOPixelShaderParameters, followed by dynamic textures & samplers). */
	OPENCOLORIO_API TSharedPtr<const FShaderParametersMetadata, ESPMode::ThreadSafe> GetShaderParamMetadata() const;

	/**
	 * Should the shader for this color transform with the given platform, shader type and vertex
	 * factory type combination be compiled
	 *
	 * @param InPlatform		The platform currently being compiled for
	 * @param InShaderType		Which shader is being compiled
	 *
	 * @return true if the shader should be compiled
	 */
	OPENCOLORIO_API virtual bool ShouldCache(EShaderPlatform InPlatform, const FShaderType* InShaderType) const;

	OPENCOLORIO_API void SerializeShaderMap(FArchive& Ar);

	/** Releases this color transform's shader map.  Must only be called on ColorTransforms not exposed to the rendering thread! */
	OPENCOLORIO_API void ReleaseShaderMap();

	/** Discards loaded shader maps if the application can't render */
	OPENCOLORIO_API void DiscardShaderMap();

	OPENCOLORIO_API void GetDependentShaderTypes(EShaderPlatform InPlatform, TArray<FShaderType*>& OutShaderTypes) const;
	OPENCOLORIO_API virtual void GetShaderMapId(EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, FOpenColorIOShaderMapId& OutId) const;

	OPENCOLORIO_API void Invalidate();

	/**
	 * Should shaders compiled for this color transform be saved to disk?
	 */
	virtual bool IsPersistent() const { return true; }

#if WITH_EDITOR
	/**
	 * Cancels all outstanding compilation jobs
	 */
	OPENCOLORIO_API void CancelCompilation();

	/**
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	OPENCOLORIO_API void FinishCompilation();
#endif // WITH_EDITOR

	/**
	 * Checks if the compilation for this shader is finished
	 *
	 * @return returns true if compilation is complete false otherwise
	 */
	OPENCOLORIO_API bool IsCompilationFinished() const;

	// Accessors.
	const TArray<FString>& GetCompileErrors() const { return CompileErrors; }
	void SetCompileErrors(const TArray<FString>& InCompileErrors) { CompileErrors = InCompileErrors; }

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	FOpenColorIOShaderMap* GetGameThreadShaderMap() const
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		return GameThreadShaderMap;
	}

	/** Returns owner name for tracking */
	FName GetOwnerFName() const
	{
		return AssetPath;
	}

	/** Note: SetRenderingThreadShaderMap must also be called with the same value, but from the rendering thread. */
	void SetGameThreadShaderMap(FOpenColorIOShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
	}

	/** Note: SetGameThreadShaderMap must also be called with the same value, but from the game thread. */
	OPENCOLORIO_API void SetRenderingThreadShaderMap(FOpenColorIOShaderMap* InShaderMap);

#if WITH_EDITOR
	void AddCompileId(uint32 InIdentifier) 
	{
		OutstandingCompileShaderMapIds.Add(InIdentifier);
	}
#endif // WITH_EDITOR

	void SetInlineShaderMap(FOpenColorIOShaderMap* InShaderMap)
	{
		checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		GameThreadShaderMap = InShaderMap;
		if (LIKELY(GameThreadShaderMap))
		{
			GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
		}
		bContainsInlineShaders = true;
		bLoadedCookedShaderMapId = true;
		CookedShaderMapId = InShaderMap->GetShaderMapId();
	}

#if WITH_EDITOR
	OPENCOLORIO_API void RemoveOutstandingCompileId(const int32 InOldOutstandingCompileShaderMapId);
#endif // WITH_EDITOR

	/**
	* Get OCIO generated source code for the shader
	* @param OutSource - generated source code
	* @return - true on Success
	*/
	bool GetColorTransformHLSLSource(FString& OutSource) 
	{
		OutSource = ShaderCode;
		return true;
	};

	const FString& GetFriendlyName()	const { return FriendlyName; }

	// Note: Takes ownership of the shader parameter metadata.
	OPENCOLORIO_API void SetupResource(
		ERHIFeatureLevel::Type InFeatureLevel,
		const FString& InShaderCodeHash,
		const FString& InShadercode,
		const FString& InRawConfigHash,
		const FString& InFriendlyName,
		const FName& InAssetPath,
		FShaderParametersMetadata* InShaderParameterMetadata
	);

	void SetCompileErrors(TArray<FString> &InErrors)
	{
		CompileErrors = InErrors;
	}
	
	FString ShaderCode;

	template <typename ShaderType>
	TShaderRef<ShaderType> GetShader() const
	{
		check(!GIsThreadedRendering || !IsInGameThread());
		if (!GIsEditor || RenderingThreadShaderMap)
		{
			return RenderingThreadShaderMap->GetShader<ShaderType>();
		}
		return TShaderRef<ShaderType>();
	};


	template <typename ShaderType>
	TShaderRef<ShaderType> GetShaderGameThread() const
	{
		if (GameThreadShaderMap)
		{
			return GameThreadShaderMap->GetShader<ShaderType>();
		}

		return TShaderRef<ShaderType>();
	};

	
	OPENCOLORIO_API bool IsSame(const FOpenColorIOShaderMapId& InId) const;

protected:
#if WITH_EDITOR
	/**
	 * Fills the passed array with IDs of shader maps unfinished compilation jobs.
	 */
	OPENCOLORIO_API void GetShaderMapIDsWithUnfinishedCompilation(TArray<int32>& OutShaderMapIds);
#endif // WITH_EDITOR

	void SetFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel)
	{
		FeatureLevel = InFeatureLevel;
	}

private:

	TArray<FString> CompileErrors;

	/** 
	 * Game thread tracked shader map, which is ref counted and manages shader map lifetime. 
	 * The shader map uses deferred deletion so that the rendering thread has a chance to process a release command when the shader map is no longer referenced.
	 * Code that sets this is responsible for updating RenderingThreadShaderMap in a thread safe way.
	 * During an async compile, this will be NULL and will not contain the actual shader map until compilation is complete.
	 */
	TRefCountPtr<FOpenColorIOShaderMap> GameThreadShaderMap;

	/**
	 * The shader parameter metadata for the generated shader (built-in FOpenColorIOPixelShaderParameters, followed by dynamic textures & samplers).
	 */
	TSharedPtr<const FShaderParametersMetadata, ESPMode::ThreadSafe> ShaderParametersMetadata;

	/** 
	 * Shader map for this FOpenColorIOTransformResource which is accessible by the rendering thread. 
	 * This must be updated along with GameThreadShaderMap, but on the rendering thread.
	 */
	FOpenColorIOShaderMap* RenderingThreadShaderMap;

	/** 
	 * Hash computed from shader code and raw config hashes. If either raw config or shader code is changed new data will
	 * be generated and saved in DDC.
	 */
	FString ShaderCodeAndConfigHash;

#if WITH_EDITOR
	/** 
	 * Contains the compiling id of this shader map when it is being compiled asynchronously. 
	 * This can be used to access the shader map during async compiling, since GameThreadShaderMap will not have been set yet.
	 */
	TArray<int32, TInlineAllocator<1> > OutstandingCompileShaderMapIds;
#endif // WITH_EDITOR

	/** Feature level that this color transform is representing. */
	ERHIFeatureLevel::Type FeatureLevel;

	/**
	 * Whether this ColorTransform was loaded with shaders inlined.
	 * If true, GameThreadShaderMap will contain a reference to the inlined shader map between Serialize and PostLoad.
	 */

	uint32 bContainsInlineShaders : 1;
	uint32 bLoadedCookedShaderMapId : 1;
	FOpenColorIOShaderMapId CookedShaderMapId;

#if WITH_EDITOR
	/**
	 * Compiles this color transform for InPlatform, storing the result in OutShaderMap if the compile was synchronous
	 */
	OPENCOLORIO_API bool BeginCompileShaderMap(
		const FOpenColorIOShaderMapId& InShaderMapId,
		EShaderPlatform InPlatform, 
		TRefCountPtr<class FOpenColorIOShaderMap>& OutShaderMap, 
		bool bApplyCompletedShaderMapForRendering,
		bool bSynchronous = false);

	/** Populates OutEnvironment with defines needed to compile shaders for this color transform. */
	OPENCOLORIO_API void SetupShaderCompilationEnvironment(
		EShaderPlatform InPlatform,
		FShaderCompilerEnvironment& OutEnvironment
		) const;
#endif // WITH_EDITOR

	FString FriendlyName;

	/** Asset using this resource */
	FName AssetPath;

	friend class FOpenColorIOShaderMap;
	friend class FShaderCompilingManager;
};


