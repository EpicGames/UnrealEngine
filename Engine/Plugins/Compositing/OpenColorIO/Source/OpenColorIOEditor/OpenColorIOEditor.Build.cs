// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenColorIOEditor : ModuleRules
	{
		public OpenColorIOEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"DesktopWidgets",
					"EditorFramework",
					"Engine",
					"ImageCore",
					"InputCore",
					"LevelEditor",
					"OpenColorIOWrapper",
					"OpenColorIO",
					"Projects",
					"PropertyEditor",
					"Renderer",
					"RenderCore",
					"RHI",
					"Settings",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
				});

			PrivateIncludePaths.AddRange(
				new string[] {
				});
		}
	}
}
