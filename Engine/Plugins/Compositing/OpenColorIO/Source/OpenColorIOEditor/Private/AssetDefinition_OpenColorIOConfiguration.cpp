// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_OpenColorIOConfiguration.h"

#include "OpenColorIOConfiguration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_OpenColorIOConfiguration)

#define LOCTEXT_NAMESPACE "AssetDefinition_OpenColorIOConfiguration"


/* FAssetTypeActions_Base interface
*****************************************************************************/

FText UAssetDefinition_OpenColorIOConfiguration::GetAssetDescription(const struct FAssetData& AssetData) const
{
	return FText::FromString(AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, ConfigurationFile)));
}


TSoftClassPtr<UObject> UAssetDefinition_OpenColorIOConfiguration::GetAssetClass() const
{
	return UOpenColorIOConfiguration::StaticClass();
}


TConstArrayView<FAssetCategoryPath> UAssetDefinition_OpenColorIOConfiguration::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(LOCTEXT("OpenColorIOConfiguration_AssetCategory", "Virtual Production")) };
	return Categories;
}

FText UAssetDefinition_OpenColorIOConfiguration::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_OpenColorIOConfiguration", "OpenColorIO Configuration");
}

FLinearColor UAssetDefinition_OpenColorIOConfiguration::GetAssetColor() const
{
	return FLinearColor::White;
}


#undef LOCTEXT_NAMESPACE
