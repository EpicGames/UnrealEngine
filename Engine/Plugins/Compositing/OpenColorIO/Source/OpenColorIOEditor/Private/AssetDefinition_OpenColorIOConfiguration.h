// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_OpenColorIOConfiguration.generated.h"

UCLASS()
class UAssetDefinition_OpenColorIOConfiguration : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// Begin UAssetDefinition interface
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return false; }
	
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	// End UAssetDefinition interface
};
