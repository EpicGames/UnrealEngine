// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOConfigurationFactory.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "Misc/Paths.h"
#include "OpenColorIOConfiguration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOConfigurationFactory)


/* UOpenColorIOConfigAssetFactoryNew structors
 *****************************************************************************/

UOpenColorIOConfigurationFactory::UOpenColorIOConfigurationFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UOpenColorIOConfiguration::StaticClass();
	bEditorImport = true;
	Formats.Add(TEXT("ocio;OpenColorIO Config File"));
}


/* UFactory overrides
 *****************************************************************************/

UObject* UOpenColorIOConfigurationFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("ocio"));
	UOpenColorIOConfiguration* Asset = NewObject<UOpenColorIOConfiguration>(InParent, InClass, InName, Flags);

	if (Asset)
	{
		FString ConfigFilename = CurrentFilename;
		FPaths::MakePathRelativeTo(ConfigFilename, *FPaths::ProjectDir());

		Asset->SetConfigurationFile(FFilePath{ ConfigFilename });

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Asset);
	}
	else
	{
		bOutOperationCanceled = true;
	}

	return Asset;
}

bool UOpenColorIOConfigurationFactory::FactoryCanImport(const FString& Filename)
{
	return FPaths::GetExtension(Filename) == TEXT("ocio");
}

uint32 UOpenColorIOConfigurationFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}
