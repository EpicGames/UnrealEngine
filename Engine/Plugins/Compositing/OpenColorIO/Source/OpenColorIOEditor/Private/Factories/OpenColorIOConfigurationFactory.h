// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "OpenColorIOConfigurationFactory.generated.h"


/**
 * Implements a factory for UOpenColorIOConfiguration objects.
 */
UCLASS(hidecategories=Object)
class UOpenColorIOConfigurationFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual uint32 GetMenuCategories() const override;
};
