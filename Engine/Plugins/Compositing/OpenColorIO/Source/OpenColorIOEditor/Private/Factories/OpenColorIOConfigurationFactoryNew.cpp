// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOConfigurationFactoryNew.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "IAssetTools.h"
#include "OpenColorIOConfiguration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOConfigurationFactoryNew)


/* UOpenColorIOConfigAssetFactoryNew structors
 *****************************************************************************/

UOpenColorIOConfigurationFactoryNew::UOpenColorIOConfigurationFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UOpenColorIOConfiguration::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UOpenColorIOConfigurationFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UOpenColorIOConfiguration>(InParent, InClass, InName, Flags);
}


uint32 UOpenColorIOConfigurationFactoryNew::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}


bool UOpenColorIOConfigurationFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

