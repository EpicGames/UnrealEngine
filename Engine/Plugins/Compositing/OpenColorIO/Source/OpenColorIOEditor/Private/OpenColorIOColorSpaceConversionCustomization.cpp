// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOColorSpaceConversionCustomization.h"

#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOSettings.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/SOpenColorIOColorSpacePicker.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "OpenColorIOColorSpaceConversionCustomization"

void FOpenColorIOColorConversionSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ColorSpaceSettingsProperty = InPropertyHandle;

	if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
	{
		ColorSpaceConversion->ValidateColorSpaces();

		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

		HeaderRow.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(512)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(MakeAttributeLambda([this]
						{
							if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
							{
								if (ColorSpaceConversion->IsValid())
								{
									return FText::FromString(*ColorSpaceConversion->ToString());
								}
							}

							return FText::FromString(TEXT("<Invalid Conversion>"));

						}))
				]
			].IsEnabled(MakeAttributeLambda([InPropertyHandle, PropertyUtils] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
	}
}

void FOpenColorIOColorConversionSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings();
	if (ColorSpaceConversion == nullptr)
	{
		return;
	}

	uint32 NumberOfChild;
	if (InStructPropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildHandle = InStructPropertyHandle->GetChildHandle(Index).ToSharedRef();

			if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, ConfigurationSource))
			{
				StructBuilder.AddProperty(ChildHandle).IsEnabled(true).ShowPropertyButtons(false);

				const TSharedRef<IPropertyUtilities> PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities().ToSharedRef();

				ChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, PropertyUtilities]
					{
						if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
						{
							ColorSpaceConversion->Reset();

							TransformPicker[OCIO_Src]->SetConfiguration(ColorSpaceConversion->ConfigurationSource);
							TransformPicker[OCIO_Dst]->SetConfiguration(ColorSpaceConversion->ConfigurationSource);
						}

						PropertyUtilities->ForceRefresh();
					}));
			}
			else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, SourceColorSpace))
			{
				SourceColorSpaceProperty = ChildHandle;
			}
			else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, DestinationColorSpace))
			{
				DestinationColorSpaceProperty = ChildHandle;
			}
			else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, DestinationDisplayView))
			{
				DestinationDisplayViewProperty = ChildHandle;
			}
			else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, DisplayViewDirection))
			{
				DisplayViewDirectionProperty = ChildHandle;
			}
		}

		// Note: ChildHandle->SetOnPropertyValueChanged isn't automatically called on parent changes, so we explicitely reset the configuration here.
		InStructPropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FOpenColorIOColorConversionSettingsCustomization::OnConfigurationReset));
		if (TSharedPtr<IPropertyHandle> ParentHandle = InStructPropertyHandle->GetParentHandle())
		{
			ParentHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FOpenColorIOColorConversionSettingsCustomization::OnConfigurationReset));
		}

		// Source color space picker widget
		TransformPicker[OCIO_Src] =
			SNew(SOpenColorIOColorSpacePicker)
			.Config(ColorSpaceConversion->ConfigurationSource)
			.Selection_Lambda([this]() -> FText {
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						return FText::FromString(ColorSpaceConversion->GetSourceString());
					}
					
					return {};
				})
			.SelectionRestriction_Lambda([this]() -> FString {
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						return ColorSpaceConversion->GetDestinationString();
					}
					
					return {};
				})
			.IsDestination(false)
			.OnColorSpaceChanged(FOnColorSpaceChanged::CreateSP(this, &FOpenColorIOColorConversionSettingsCustomization::OnSelectionChanged));

		// Destination color space picker widget
		TransformPicker[OCIO_Dst] =
			SNew(SOpenColorIOColorSpacePicker)
			.Config(ColorSpaceConversion->ConfigurationSource)
			.Selection_Lambda([this]() -> FText {
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						return FText::FromString(ColorSpaceConversion->GetDestinationString());
					}
					
					return {};
				})
			.SelectionRestriction_Lambda([this]() -> FString {
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						return ColorSpaceConversion->GetSourceString();
					}

					return {};
				})
			.IsDestination(true)
			.OnColorSpaceChanged(FOnColorSpaceChanged::CreateSP(this, &FOpenColorIOColorConversionSettingsCustomization::OnSelectionChanged));

		// Source color space picker widget
		StructBuilder.AddCustomRow(LOCTEXT("TransformSource", "Transform Source"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TransformSource", "Transform Source"))
			.ToolTipText(LOCTEXT("TransformSource_Tooltip", "The source color space used for the transform."))
			.Font(StructCustomizationUtils.GetRegularFont())
		]
		.ValueContent()
		.MaxDesiredWidth(512)
		[
			TransformPicker[OCIO_Src].ToSharedRef()
		];

		StructBuilder.AddCustomRow(LOCTEXT("TransformDestination", "Transform Destination"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TransformDestination", "Transform Destination"))
			.ToolTipText(LOCTEXT("TransformDestination_Tooltip", "The destination color space used for the transform."))
			.Font(StructCustomizationUtils.GetRegularFont())
		]
		.ValueContent()
		.MaxDesiredWidth(512)
		[
			TransformPicker[OCIO_Dst].ToSharedRef()
		];

		const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();
		if (Settings->bSupportInverseViewTransforms)
		{
			StructBuilder.AddCustomRow(LOCTEXT("InvertViewTransform", "Invert View Transform"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InvertViewTransform", "Invert View Transform"))
				.ToolTipText(LOCTEXT("InvertViewTransform_Tooltip", "Option to invert the display-view transform."))
				.Font(StructCustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.MaxDesiredWidth(512)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						if (ColorSpaceConversion->IsDisplayView())
						{
							return ColorSpaceConversion->DisplayViewDirection == EOpenColorIOViewTransformDirection::Inverse ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					if (DisplayViewDirectionProperty.IsValid())
					{
						DisplayViewDirectionProperty->SetValue((NewState == ECheckBoxState::Checked) ? (uint8)EOpenColorIOViewTransformDirection::Inverse : (uint8)EOpenColorIOViewTransformDirection::Forward);
					}
				})
				.IsEnabled_Lambda([this]()
				{
					if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
					{
						return ColorSpaceConversion->IsDisplayView();
					}

					return false;
				})
			];
		}

		if (IsValid(ColorSpaceConversion->ConfigurationSource))
		{
			const TArray<UObject*> ConfigurationObjects = { ColorSpaceConversion->ConfigurationSource.Get() };
			
			IDetailPropertyRow* ContextRow = StructBuilder.AddExternalObjectProperty(ConfigurationObjects, GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, Context));
			if (ContextRow)
			{
				ContextRow->DisplayName(LOCTEXT("BaseContext", "Base Context"));
				ContextRow->IsEnabled(false);
				ContextRow->Visibility(MakeAttributeLambda([this]()
					{
						if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
						{
							if (IsValid(ColorSpaceConversion->ConfigurationSource) && !ColorSpaceConversion->ConfigurationSource->Context.IsEmpty())
							{
								return EVisibility::Visible;
							}
						}

						return EVisibility::Hidden;
					})
				);
			}
		}
	}

	InStructPropertyHandle->MarkHiddenByCustomization();
}

void FOpenColorIOColorConversionSettingsCustomization::OnSelectionChanged(const FOpenColorIOColorSpace& NewColorSpace, const FOpenColorIODisplayView& NewDisplayView, bool bIsDestination)
{
	TArray<TSharedPtr<IPropertyHandle>> Properties = { SourceColorSpaceProperty, DestinationColorSpaceProperty, DestinationDisplayViewProperty };

	const FScopedTransaction Transaction(LOCTEXT("OCIOConfigurationSelectionUpdate", "OCIO Configuration Selection Update"));

	// Notify outer object(s). Only called for one property since it is shared.
	TArray<UObject*> Objects;
	Properties[0]->GetOuterObjects(Objects);
	for (UObject* OuterObject : Objects)
	{
		OuterObject->Modify();
	}

	for (const TSharedPtr<IPropertyHandle>& Property : Properties)
	{
		check(Property.IsValid());
		Property->NotifyPreChange();
	}

	if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
	{
		if (bIsDestination)
		{
			if (NewDisplayView.IsValid())
			{
				ColorSpaceConversion->DestinationColorSpace.Reset();
				ColorSpaceConversion->DestinationDisplayView = NewDisplayView;
			}
			else
			{
				ColorSpaceConversion->DestinationDisplayView.Reset();
				ColorSpaceConversion->DestinationColorSpace = NewColorSpace;
			}
		}
		else
		{
			ColorSpaceConversion->SourceColorSpace = NewColorSpace;
		}
	}

	for (const TSharedPtr<IPropertyHandle>& Property : Properties)
	{
		Property->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
	for (const TSharedPtr<IPropertyHandle>& Property : Properties)
	{
		Property->NotifyFinishedChangingProperties();
	}
}

void FOpenColorIOColorConversionSettingsCustomization::OnConfigurationReset()
{
	TransformPicker[OCIO_Src]->SetConfiguration(nullptr);
	TransformPicker[OCIO_Dst]->SetConfiguration(nullptr);

	if (FOpenColorIOColorConversionSettings* ColorSpaceConversion = GetConversionSettings())
	{
		ColorSpaceConversion->Reset();
	}
}

FOpenColorIOColorConversionSettings* FOpenColorIOColorConversionSettingsCustomization::GetConversionSettings() const
{
	if (ColorSpaceSettingsProperty->IsValidHandle())
	{
		void* Data = nullptr;
		if (ColorSpaceSettingsProperty->GetValueData(Data) == FPropertyAccess::Success)
		{
			return static_cast<FOpenColorIOColorConversionSettings*>(Data);
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
