// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SOpenColorIOColorSpacePicker.h"
#include "Widgets/SWidget.h"

/**
 * Implements a details view customization for the FOpenColorIOColorConversionSettings
 */
class FOpenColorIOColorConversionSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FOpenColorIOColorConversionSettingsCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

	void OnSelectionChanged(const FOpenColorIOColorSpace& NewColorSpace, const FOpenColorIODisplayView& NewDisplayView, bool bIsDestination);

private:
	/** Callback to reset the configuration in the transform source/destination pickers. */
	void OnConfigurationReset();

	/** Get the struct value of the color space settings member property.*/
	struct FOpenColorIOColorConversionSettings* GetConversionSettings() const;

	/** Pointer to the struct SourceColorSpace property handle. */
	TSharedPtr<IPropertyHandle> SourceColorSpaceProperty;
	
	/** Pointer to the struct DestinationColorSpace property handle. */
	TSharedPtr<IPropertyHandle> DestinationColorSpaceProperty;

	/** Pointer to the struct DestinationDisplayView property handle. */
	TSharedPtr<IPropertyHandle> DestinationDisplayViewProperty;

	/** Pointer to the struct DisplayViewDirection property handle. */
	TSharedPtr<IPropertyHandle> DisplayViewDirectionProperty;

	/** ColorSpace pickers reference to update them when config asset is changed */
	TStaticArray<TSharedPtr<SOpenColorIOColorSpacePicker>, 2> TransformPicker;

	/** Property handle to the conversion settings struct. */
	TSharedPtr<IPropertyHandle> ColorSpaceSettingsProperty;
};
