// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOColorSpaceCustomization.h"

#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOWrapper.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "OpenColorIOColorSpaceCustomization"

void FOpenColorIOColorSpaceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	//Reset internals
	CachedProperty = InPropertyHandle;

	if (CachedProperty->GetNumPerObjectValues() == 1 && CachedProperty->IsValidHandle())
	{
		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

		HeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(512)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(MakeAttributeLambda([WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)]
						{
							TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
							if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
							{
								TArray<void*> RawData;
								PropertyHandle->AccessRawData(RawData);
								check(RawData.Num() == 1);

								const FOpenColorIOColorSpace* ColorSpaceValue = reinterpret_cast<FOpenColorIOColorSpace*>(RawData[0]);
								if (ColorSpaceValue != nullptr)
								{
									const FString ColorSpaceName = ColorSpaceValue->ToString();

									if (!ColorSpaceName.IsEmpty())
									{
										return FText::FromString(ColorSpaceName);
									}
								}
							}
							
							return LOCTEXT("None", "<None>");
						}))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FOpenColorIOColorSpaceCustomization::HandleSourceComboButtonMenuContent)
					.ContentPadding(FMargin(4.0, 2.0))
				]
			].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
	}
}

void FOpenColorIODisplayViewCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	//Reset internals
	CachedProperty = InPropertyHandle;

	if (CachedProperty->GetNumPerObjectValues() == 1 && CachedProperty->IsValidHandle())
	{
		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

		HeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
		.ValueContent()
			.MaxDesiredWidth(512)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(MakeAttributeLambda([WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)]
						{
							TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
							if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
							{
								FProperty* Property = PropertyHandle->GetProperty();
								check(Property && CastField<FStructProperty>(Property) && CastField<FStructProperty>(Property)->Struct && CastField<FStructProperty>(Property)->Struct->IsChildOf(FOpenColorIODisplayView::StaticStruct()));

								TArray<void*> RawData;
								PropertyHandle->AccessRawData(RawData);
								check(RawData.Num() == 1);

								const FOpenColorIODisplayView* DisplayViewValue = reinterpret_cast<const FOpenColorIODisplayView*>(RawData[0]);
								if (DisplayViewValue != nullptr)
								{
									const FString DisplayViewName = DisplayViewValue->ToString();

									if (!DisplayViewName.IsEmpty())
									{
										return FText::FromString(DisplayViewName);
									}
								}
							}

							return LOCTEXT("None", "<None>");
						}))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FOpenColorIODisplayViewCustomization::HandleSourceComboButtonMenuContent)
					.ContentPadding(FMargin(4.0, 2.0))
				]
			].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
	}
}

IPropertyTypeCustomizationOpenColorIO::IPropertyTypeCustomizationOpenColorIO(TSharedPtr<IPropertyHandle> InConfigurationObjectProperty)
	: ConfigurationObjectProperty(MoveTemp(InConfigurationObjectProperty))
{
}

FOpenColorIOWrapperConfig* IPropertyTypeCustomizationOpenColorIO::GetConfigWrapper(const TSharedPtr<IPropertyHandle>& InConfigurationAssetProperty)
{
	if (InConfigurationAssetProperty.IsValid())
	{
		TArray<void*> RawData;
		InConfigurationAssetProperty->AccessRawData(RawData);
		check(RawData.Num() == 1);
		
		UOpenColorIOConfiguration* Configuration = reinterpret_cast<UOpenColorIOConfiguration*>(RawData[0]);
		check(Configuration);
		
		return Configuration->GetConfigWrapper();
	}

	return nullptr;
}

void FOpenColorIOColorSpaceCustomization::ProcessColorSpaceForMenuGeneration(FMenuBuilder& InMenuBuilder, const int32 InMenuDepth, const FString& InPreviousFamilyHierarchy, const FOpenColorIOColorSpace& InColorSpace, TArray<FString>& InOutExistingMenuFilter)
{
	const FString NextFamilyName = InColorSpace.GetFamilyNameAtDepth(InMenuDepth);
	if (!NextFamilyName.IsEmpty())
	{
		if (!InOutExistingMenuFilter.Contains(NextFamilyName))
		{
			//Only add the previous family and delimiter if there was one. First family doesn't need it.
			const FString PreviousHierarchyToAdd = !InPreviousFamilyHierarchy.IsEmpty() ? InPreviousFamilyHierarchy + FOpenColorIOColorSpace::FamilyDelimiter : TEXT("");
			const FString NewHierarchy = PreviousHierarchyToAdd + NextFamilyName;;
			const int32 NextMenuDepth = InMenuDepth + 1;
			InMenuBuilder.AddSubMenu(
				FText::FromString(NextFamilyName),
				LOCTEXT("OpensFamilySubMenu", "ColorSpace Family Sub Menu"),
				FNewMenuDelegate::CreateRaw(this, &FOpenColorIOColorSpaceCustomization::PopulateSubMenu, NextMenuDepth, NewHierarchy)
			);

			InOutExistingMenuFilter.Add(NextFamilyName);
		}
	}
	else
	{
		AddMenuEntry(InMenuBuilder, InColorSpace);
	}
}

void FOpenColorIOColorSpaceCustomization::PopulateSubMenu(FMenuBuilder& InMenuBuilder, int32 InMenuDepth, FString InPreviousFamilyHierarchy)
{
	//Submenus should always be at a certain depth level
	check(InMenuDepth > 0);

	// To keep track of submenus that were already added
	TArray<FString> ExistingSubMenus;

	const FOpenColorIOWrapperConfig* ConfigWrapper = GetConfigWrapper(ConfigurationObjectProperty);

	if (ConfigWrapper)
	{
		for (int32 i = 0; i < ConfigWrapper->GetNumColorSpaces(); ++i)
		{
			FOpenColorIOColorSpace ColorSpace;
			ColorSpace.ColorSpaceName = ConfigWrapper->GetColorSpaceName(i);
			ColorSpace.FamilyName = ConfigWrapper->GetColorSpaceFamilyName(*ColorSpace.ColorSpaceName);
			ColorSpace.Description	= ConfigWrapper->GetColorSpaceDescription(*ColorSpace.ColorSpaceName);

			//Filter out color spaces that don't belong to this hierarchy
			if (InPreviousFamilyHierarchy.IsEmpty() || ColorSpace.FamilyName.Contains(InPreviousFamilyHierarchy))
			{
				ProcessColorSpaceForMenuGeneration(InMenuBuilder, InMenuDepth, InPreviousFamilyHierarchy, ColorSpace, ExistingSubMenus);
			}
		}
	}

}

void FOpenColorIOColorSpaceCustomization::AddMenuEntry(FMenuBuilder& InMenuBuilder, const FOpenColorIOColorSpace& InColorSpace)
{
	InMenuBuilder.AddMenuEntry(
		FText::FromString(InColorSpace.ToString()),
		FText::FromString(InColorSpace.Description),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, InColorSpace]
			{
				if (FStructProperty* StructProperty = CastField<FStructProperty>(CachedProperty->GetProperty()))
				{
					TArray<void*> RawData;
					CachedProperty->AccessRawData(RawData);
					FOpenColorIOColorSpace* PreviousColorSpaceValue = reinterpret_cast<FOpenColorIOColorSpace*>(RawData[0]);

					FString TextValue;
					StructProperty->Struct->ExportText(TextValue, &InColorSpace, PreviousColorSpaceValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
					ensure(CachedProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
				}
			}
			),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, InColorSpace]
			{
				TArray<void*> RawData;
				CachedProperty->AccessRawData(RawData);
				FOpenColorIOColorSpace* ColorSpaceValue = reinterpret_cast<FOpenColorIOColorSpace*>(RawData[0]);
				return *ColorSpaceValue == InColorSpace;
			})
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
		);
}

TSharedRef<SWidget> FOpenColorIOColorSpaceCustomization::HandleSourceComboButtonMenuContent()
{
	const FOpenColorIOWrapperConfig* ConfigWrapper = GetConfigWrapper(ConfigurationObjectProperty);

	// Generate menu
	FMenuBuilder MenuBuilder(true, nullptr);
	TArray<FString> ExistingSubMenus;

	MenuBuilder.BeginSection("AllColorSpaces", LOCTEXT("AllColorSpacesSection", "ColorSpaces"));
	{
		if (ConfigWrapper)
		{
			const int32 ColorSpaceCount = ConfigWrapper->GetNumColorSpaces();

			for (int32 i = 0; i < ColorSpaceCount; ++i)
			{
				FOpenColorIOColorSpace ColorSpace;
				ColorSpace.ColorSpaceName = ConfigWrapper->GetColorSpaceName(i);
				ColorSpace.FamilyName = ConfigWrapper->GetColorSpaceFamilyName(*ColorSpace.ColorSpaceName);
				ColorSpace.Description = ConfigWrapper->GetColorSpaceDescription(*ColorSpace.ColorSpaceName);

				//Top level menus have no preceding hierarchy.
				const int32 CurrentMenuDepth = 0;
				const FString PreviousHierarchy = FString();
				ProcessColorSpaceForMenuGeneration(MenuBuilder, CurrentMenuDepth, PreviousHierarchy, ColorSpace, ExistingSubMenus);
			}

			if (ColorSpaceCount <= 0)
			{
				MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoColorSpaceFound", "No color space found"), false, false);
			}
		}
		else
		{
			MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("InvalidConfigurationFile", "Invalid configuration file"), false, false);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FOpenColorIODisplayViewCustomization::PopulateViewSubMenu(FMenuBuilder& InMenuBuilder, FOpenColorIODisplayView InDisplayView)
{
	const FOpenColorIOWrapperConfig* ConfigWrapper = GetConfigWrapper(ConfigurationObjectProperty);

	if (ConfigWrapper)
	{
		for (int32 i = 0; i < ConfigWrapper->GetNumViews(*InDisplayView.Display); ++i)
		{
			InDisplayView.View = ConfigWrapper->GetViewName(*InDisplayView.Display, i);
			InDisplayView.Description = ConfigWrapper->GetDisplayViewTransformDescription(*InDisplayView.Display, *InDisplayView.View);

			AddMenuEntry(InMenuBuilder, InDisplayView);
		}
	}
}

void FOpenColorIODisplayViewCustomization::AddMenuEntry(FMenuBuilder& InMenuBuilder, const FOpenColorIODisplayView& InDisplayView)
{
	InMenuBuilder.AddMenuEntry(
		FText::FromString(InDisplayView.View),
		FText::FromString(InDisplayView.Description),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, InDisplayView]
				{
					if (FStructProperty* StructProperty = CastField<FStructProperty>(CachedProperty->GetProperty()))
					{
						TArray<void*> RawData;
						CachedProperty->AccessRawData(RawData);
						FOpenColorIODisplayView* PreviousDisplayViewValue = reinterpret_cast<FOpenColorIODisplayView*>(RawData[0]);

						FString TextValue;
						StructProperty->Struct->ExportText(TextValue, &InDisplayView, PreviousDisplayViewValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
						ensure(CachedProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
					}
				}
			),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, InDisplayView]
				{
					TArray<void*> RawData;
					CachedProperty->AccessRawData(RawData);
					FOpenColorIODisplayView* DisplayViewValue = reinterpret_cast<FOpenColorIODisplayView*>(RawData[0]);
					return *DisplayViewValue == InDisplayView;
				})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
}

TSharedRef<SWidget> FOpenColorIODisplayViewCustomization::HandleSourceComboButtonMenuContent()
{
	const FOpenColorIOWrapperConfig* ConfigWrapper = GetConfigWrapper(ConfigurationObjectProperty);

	// Generate menu
	FMenuBuilder MenuBuilder(true, nullptr);
	TArray<FString> ExistingSubMenus;

	MenuBuilder.BeginSection("AllDisplayViews", LOCTEXT("AllDisplayViewsSection", "Display - View"));
	{
		if (ConfigWrapper)
		{
			const int32 DisplayCount = ConfigWrapper->GetNumDisplays();

			for (int32 i = 0; i < DisplayCount; ++i)
			{
				const FOpenColorIODisplayView DisplayViewValue = { ConfigWrapper->GetDisplayName(i), FStringView{} };

				MenuBuilder.AddSubMenu(
					FText::FromString(DisplayViewValue.Display),
					LOCTEXT("OpensDisplayViewSubMenu", "Display - View Family Sub Menu"),
					FNewMenuDelegate::CreateRaw(this, &FOpenColorIODisplayViewCustomization::PopulateViewSubMenu, DisplayViewValue)
				);
			}

			if (DisplayCount <= 0)
			{
				MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoDisplayFound", "No display found"), false, false);
			}
		}
		else
		{
			MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("InvalidConfigurationFile", "Invalid configuration file"), false, false);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
