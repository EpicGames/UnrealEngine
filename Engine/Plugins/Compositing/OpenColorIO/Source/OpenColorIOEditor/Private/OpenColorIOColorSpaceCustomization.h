// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "OpenColorIOColorSpace.h"
#include "Widgets/SWidget.h"

class FOpenColorIOWrapperConfig;

/**
 * Implements a details view customization for the FOpenColorIOConfiguration
 */
class IPropertyTypeCustomizationOpenColorIO : public IPropertyTypeCustomization
{
public:
	IPropertyTypeCustomizationOpenColorIO(TSharedPtr<IPropertyHandle> InConfigurationObjectProperty);

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override {}

protected:

	static FOpenColorIOWrapperConfig* GetConfigWrapper(const TSharedPtr<IPropertyHandle>& InConfigurationObjectProperty);

protected:

	/** Pointer to the property handle. */
	TSharedPtr<IPropertyHandle> CachedProperty;

	/** Pointer to the configuration object property handle. */
	TSharedPtr<IPropertyHandle> ConfigurationObjectProperty;
};


class FOpenColorIOColorSpaceCustomization : public IPropertyTypeCustomizationOpenColorIO
{
public:
	using IPropertyTypeCustomizationOpenColorIO::IPropertyTypeCustomizationOpenColorIO;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedPtr<IPropertyHandle> InConfigurationObjectProperty)
	{
		return MakeShared<FOpenColorIOColorSpaceCustomization>(MoveTemp(InConfigurationObjectProperty));
	}

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

private:
	void ProcessColorSpaceForMenuGeneration(FMenuBuilder& InMenuBuilder, const int32 InMenuDepth, const FString& InPreviousFamilyHierarchy, const FOpenColorIOColorSpace& InColorSpace, TArray<FString>& InOutExistingMenuFilter);
	void PopulateSubMenu(FMenuBuilder& InMenuBuilder, const int32 InMenuDepth, FString InPreviousFamilyHierarchy);
	void AddMenuEntry(FMenuBuilder& InMenuBuilder, const FOpenColorIOColorSpace& InColorSpace);

	TSharedRef<SWidget> HandleSourceComboButtonMenuContent();
};


class FOpenColorIODisplayViewCustomization : public IPropertyTypeCustomizationOpenColorIO
{
public:
	using IPropertyTypeCustomizationOpenColorIO::IPropertyTypeCustomizationOpenColorIO;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedPtr<IPropertyHandle> InConfigurationObjectProperty)
	{
		return MakeShared<FOpenColorIODisplayViewCustomization>(MoveTemp(InConfigurationObjectProperty));
	}

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

private:
	void PopulateViewSubMenu(FMenuBuilder& InMenuBuilder, FOpenColorIODisplayView InDisplayView);
	void AddMenuEntry(FMenuBuilder& InMenuBuilder, const FOpenColorIODisplayView& InDisplayView);

	TSharedRef<SWidget> HandleSourceComboButtonMenuContent();
};

