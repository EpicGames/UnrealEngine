// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOConfigurationCustomization.h"

#include "OpenColorIOColorSpaceCustomization.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOWrapper.h"
#include "Containers/StringConv.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "OpenColorIOConfigurationCustomization"

void FOpenColorIOConfigurationCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() != 1)
	{
		return;
	}

	if (UOpenColorIOConfiguration* OpenColorIOConfig = Cast<UOpenColorIOConfiguration>(Objects[0].Get()))
	{
		if (FOpenColorIOWrapperConfig* Wrapper = OpenColorIOConfig->GetConfigWrapper())
		{
			constexpr bool bRecommendedOnly = false;
			constexpr bool bShortenUIName = true;
			CachedConfigNamePairs = Wrapper->GetBuiltInConfigNames(bRecommendedOnly, bShortenUIName);
		}
	}
	
	ConfigTypes.Reset(CachedConfigNamePairs.Num());

	for (const TPair<FString, FString>& Pair : CachedConfigNamePairs)
	{
		ConfigTypes.Add(MakeShared<FString>(Pair.Value));
	}

	ConfigTypes.Add(MakeShared<FString>(TEXT("External File")));
	
	ConfigurationFilePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UOpenColorIOConfiguration, ConfigurationFile));

	// Update current type
	OnConfigurationFileChanged();

	ConfigurationFilePropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FOpenColorIOConfigurationCustomization::OnConfigurationFileChanged));
	ConfigurationFilePropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FOpenColorIOConfigurationCustomization::OnConfigurationFileChanged));
	ConfigurationFilePropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FOpenColorIOConfigurationCustomization::OnConfigurationFileChanged));

	IDetailCategoryBuilder& ConfigCategory = DetailBuilder.EditCategory("Config");

	const FText TooltipTypes = LOCTEXT("ConfigTypesTooltip", "Types of OpenColorIO configuration, which facilitates the selection of built-ins (latest, or version locked).");

	ConfigCategory.AddCustomRow(LOCTEXT("ConfigTypes", "Types"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("ConfigTypeLabel", "Type"))
			.ToolTipText(TooltipTypes)
		]
		.ValueContent()
		[
			SAssignNew(TypeCombo, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&ConfigTypes)
			.OnSelectionChanged(this, &FOpenColorIOConfigurationCustomization::OnConfigTypeChanged)
			.OnGenerateWidget(this, &FOpenColorIOConfigurationCustomization::MakeConfigTypeWidget)
			.InitiallySelectedItem(CurrentConfigType)
			[
				SNew(STextBlock)
				.Text(this, &FOpenColorIOConfigurationCustomization::GetCurrentConfigTypeLabel)
			]
		];

	ConfigCategory.AddProperty(ConfigurationFilePropertyHandle);

	ConfigCategory.AddCustomRow(LOCTEXT("Config", "Reload and Rebuild"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("ButtonCategory", "Rebuild from Raw Config"))
		]
		.ValueContent()
		[
			SNew(SButton)
			.Text(LOCTEXT("ButtonName", "Reload and Rebuild"))
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("ButtonToolTip", "Reload the current OCIO config file and rebuild shaders."))
			.OnClicked_Lambda([Objects]()
			{
				if(UOpenColorIOConfiguration* OpenColorIOConfig = Cast<UOpenColorIOConfiguration>(Objects[0].Get()))
				{
					OpenColorIOWrapper::ClearAllCaches();

					OpenColorIOConfig->ReloadExistingColorspaces(true);
				}
				return FReply::Handled();
			})
		];

	TSharedPtr<IPropertyHandle> ConfigurationObjectHandle = ConfigurationFilePropertyHandle->GetParentHandle();

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
			FOpenColorIOColorSpace::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([ConfigurationObjectHandle] { return FOpenColorIOColorSpaceCustomization::MakeInstance(ConfigurationObjectHandle); }));

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FOpenColorIODisplayView::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([ConfigurationObjectHandle] { return FOpenColorIODisplayViewCustomization::MakeInstance(ConfigurationObjectHandle); }));
}

void FOpenColorIOConfigurationCustomization::OnConfigTypeChanged(TSharedPtr<FString> InValue, ESelectInfo::Type InSelectInfo)
{
	if (InSelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	CurrentConfigType = InValue;
	
	if (ConfigurationFilePropertyHandle.IsValid() && InValue.IsValid())
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(ConfigurationFilePropertyHandle->GetProperty()))
		{
			TArray<void*> RawData;
			ConfigurationFilePropertyHandle->AccessRawData(RawData);

			FFilePath* PreviousFilePath = nullptr;
			if (!RawData.IsEmpty())
			{
				PreviousFilePath = reinterpret_cast<FFilePath*>(RawData[0]);
			}

			if (PreviousFilePath)
			{
				FFilePath NewFilePath;

				if (InValue->Equals(*ConfigTypes.Last()))
				{
					// If external file, we clear the path
					NewFilePath.FilePath = TEXT("");
				}
				else
				{
					for (const TPair<FString, FString>& Pair : CachedConfigNamePairs)
					{
						if (InValue->Equals(Pair.Value))
						{
							NewFilePath.FilePath = Pair.Key;
							NewFilePath.FilePath.InsertAt(0, TEXT("ocio://"));
							break;
						}
					}
				}

				FString TextValue;
				StructProperty->Struct->ExportText(TextValue, &NewFilePath, PreviousFilePath, nullptr, EPropertyPortFlags::PPF_None, nullptr);

				ensure(ConfigurationFilePropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
			}
		}
	}
}

void FOpenColorIOConfigurationCustomization::OnConfigurationFileChanged()
{
	bool bUpdated = false;
	
	if (ConfigurationFilePropertyHandle && ConfigurationFilePropertyHandle->IsValidHandle())
	{
		FFilePath* CurrentFilePath = nullptr;

		// Update current type
		TArray<void*> RawData;
		ConfigurationFilePropertyHandle->AccessRawData(RawData);
		if (!RawData.IsEmpty())
		{
			CurrentFilePath = reinterpret_cast<FFilePath*>(RawData[0]);
		}

		if (CurrentFilePath != nullptr)
		{
			FString ResolvedPath = FOpenColorIOWrapperConfig::ResolveConfigPath(*CurrentFilePath->FilePath);
			ResolvedPath.RemoveFromStart(TEXT("ocio://"));

			if (const FString* UIName = CachedConfigNamePairs.Find(ResolvedPath))
			{
				const TSharedPtr<FString>* FoundType = ConfigTypes.FindByPredicate([UIName](const TSharedPtr<FString>& InValue) -> bool
					{
						return InValue.IsValid() ? InValue->Equals(*UIName) : false;
					});

				if (FoundType)
				{
					CurrentConfigType = *FoundType;
					bUpdated = true;
				}
			}
		}
	}

	if (!bUpdated)
	{
		CurrentConfigType = ConfigTypes.Last();
	}

	if (TypeCombo.IsValid())
	{
		TypeCombo->SetSelectedItem(CurrentConfigType);
	}
}

TSharedRef<SWidget> FOpenColorIOConfigurationCustomization::MakeConfigTypeWidget(TSharedPtr<FString> InOption)
{
	return SNew(STextBlock).Text(FText::FromString(*InOption));
}

FText FOpenColorIOConfigurationCustomization::GetCurrentConfigTypeLabel() const
{
	if (CurrentConfigType.IsValid())
	{
		return FText::FromString(*CurrentConfigType);
	}

	return LOCTEXT("Invalid", "Invalid");
}


#undef LOCTEXT_NAMESPACE
