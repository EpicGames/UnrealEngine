// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

namespace ESelectInfo { enum Type : int; }
class IPropertyHandle;
template <typename OptionType> class SComboBox;
class SWidget;
class IDetailLayoutBuilder;

/**
 * Implements a details view customization for the UOpenColorIOConfiguration
 */
class FOpenColorIOConfigurationCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FOpenColorIOConfigurationCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	/** Callback for when the config type changes. */
	void OnConfigTypeChanged(TSharedPtr<FString> InValue, ESelectInfo::Type InSelectInfo);

	/** Callback for when the configuration file path changes. */
	void OnConfigurationFileChanged();

	/** Create the config type widget. */
	TSharedRef<SWidget> MakeConfigTypeWidget(TSharedPtr<FString> InOption);

	/** Get the current config type label. */
	FText GetCurrentConfigTypeLabel() const;

	/** Cached OpenColorIO built-in name/UI-name pairs. */
	TMap<FString, FString> CachedConfigNamePairs;

	/** List of available config types */
	TArray<TSharedPtr<FString>> ConfigTypes;

	/** Currently selected type */
	TSharedPtr<FString> CurrentConfigType;

	/** Handle to the ConfigurationFile property */
	TSharedPtr<IPropertyHandle> ConfigurationFilePropertyHandle;

	/** Cached combo box widget. */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> TypeCombo;
};

