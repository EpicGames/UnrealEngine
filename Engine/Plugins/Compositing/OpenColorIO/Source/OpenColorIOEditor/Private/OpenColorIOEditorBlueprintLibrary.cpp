// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOEditorBlueprintLibrary.h"

#include "Modules/ModuleManager.h"

#include "Engine/Texture2D.h"
#include "OpenColorIOEditorModule.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIOConfiguration.h"
#include "TextureCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOEditorBlueprintLibrary)


void UOpenColorIOEditorBlueprintLibrary::SetActiveViewportConfiguration(const FOpenColorIODisplayConfiguration& InConfiguration)
{
	FOpenColorIOEditorModule& OpenColorIOEditorModule = FModuleManager::LoadModuleChecked<FOpenColorIOEditorModule>("OpenColorIOEditor");

	OpenColorIOEditorModule.SetActiveViewportConfiguration(InConfiguration);
}

bool UOpenColorIOEditorBlueprintLibrary::ApplyColorSpaceTransformToColor(const FOpenColorIOColorConversionSettings& ConversionSettings, const FLinearColor& InColor, FLinearColor& OutColor)
{
	if (IsValid(ConversionSettings.ConfigurationSource))
	{
		OutColor = InColor;

		return ConversionSettings.ConfigurationSource->TransformColor(ConversionSettings, OutColor);
	}

	return false;
}

bool UOpenColorIOEditorBlueprintLibrary::ApplyColorSpaceTransformToTexture(const FOpenColorIOColorConversionSettings& ConversionSettings, UTexture* InOutTexture)
{
	if (IsValid(InOutTexture))
	{
		return ApplyColorSpaceTransformToTextureCompressed(ConversionSettings, InOutTexture->CompressionSettings, InOutTexture);
	}

	return false;
}

bool UOpenColorIOEditorBlueprintLibrary::ApplyColorSpaceTransformToTextureCompressed(const FOpenColorIOColorConversionSettings& ConversionSettings, TextureCompressionSettings TargetCompression, UTexture* InOutTexture)
{
	if (IsValid(InOutTexture) && IsValid(ConversionSettings.ConfigurationSource))
	{
		FImage ImageMip0;
		if (InOutTexture->Source.GetMipImage(ImageMip0, 0))
		{
			bool bTransformSucceeded = ConversionSettings.ConfigurationSource->TransformImage(ConversionSettings, ImageMip0);
			if (bTransformSucceeded)
			{
				InOutTexture->PreEditChange(nullptr);
				void* TargetData = InOutTexture->Source.LockMip(0);
				FMemory::Memcpy(TargetData, ImageMip0.RawData.GetData(), ImageMip0.GetImageSizeBytes());
				InOutTexture->Source.UnlockMip(0);
				InOutTexture->CompressionSettings = TargetCompression;
				InOutTexture->PostEditChange();

				FTextureCompilingManager::Get().FinishCompilation({ InOutTexture });

				return true;
			}
		}
	}

	return false;
}
