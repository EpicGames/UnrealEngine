// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOEditorModule.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Engine/World.h"
#include "Interfaces/IPluginManager.h"
#include "IOpenColorIOModule.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "OpenColorIODisplayManager.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOEditorSettings.h"
#include "OpenColorIOColorSpaceConversionCustomization.h"
#include "OpenColorIOConfigurationCustomization.h"
#include "OpenColorIOColorTransform.h"
#include "PropertyEditorModule.h"
#include "SLevelViewport.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "ToolMenus.h"
#include "Widgets/SOpenColorIODisplay.h"

LLM_DEFINE_TAG(OpenColorIO_OpenColorIOEditor);
DEFINE_LOG_CATEGORY(LogOpenColorIOEditor);

#define LOCTEXT_NAMESPACE "OpenColorIOEditorModule"


void FOpenColorIOEditorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(OpenColorIO_OpenColorIOEditor);

	FWorldDelegates::OnPreWorldInitialization.AddRaw(this, &FOpenColorIOEditorModule::OnWorldInit);
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FOpenColorIOEditorModule::OnEngineLoopInitComplete);
	
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

	RegisterCustomizations();
	RegisterStyle();
	RegisterViewMenuExtension();
	RegisterSettings();
}

void FOpenColorIOEditorModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(OpenColorIO_OpenColorIOEditor);

	UnregisterSettings();
	UnregisterViewMenuExtension();
	UnregisterStyle();
	UnregisterCustomizations();

	CleanFeatureLevelDelegate();
	FWorldDelegates::OnPreWorldInitialization.RemoveAll(this);
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
}

void FOpenColorIOEditorModule::SetActiveViewportConfiguration(const FOpenColorIODisplayConfiguration& InConfiguration)
{
	OnDisplayConfigurationChanged(InConfiguration);
}

void FOpenColorIOEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FOpenColorIOColorConversionSettings::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FOpenColorIOColorConversionSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(UOpenColorIOConfiguration::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FOpenColorIOConfigurationCustomization::MakeInstance));
}

void FOpenColorIOEditorModule::UnregisterCustomizations()
{
	if (UObjectInitialized() == true)
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(UOpenColorIOConfiguration::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FOpenColorIOColorSpace::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FOpenColorIODisplayView::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FOpenColorIOColorConversionSettings::StaticStruct()->GetFName());
	}
}

void FOpenColorIOEditorModule::RegisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "OpenColorIO Editor",
			LOCTEXT("OpenColorIOSettingsName", "OpenColorIO Editor"),
			LOCTEXT("OpenColorIOSettingsDescription", "Configure default viewports settings for OpenColorIO."),
			GetMutableDefault<UOpenColorIODefaultViewportSettings>());
	}
}

void FOpenColorIOEditorModule::UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "OpenColorIO Editor");
	}
}

void FOpenColorIOEditorModule::OnWorldInit(UWorld* InWorld, const UWorld::InitializationValues InInitializationValues)
{
	LLM_SCOPE_BYTAG(OpenColorIO_OpenColorIOEditor);
	if (InWorld && InWorld->WorldType == EWorldType::Editor)
	{
		CleanFeatureLevelDelegate();
		EditorWorld = MakeWeakObjectPtr(InWorld);

		FOnFeatureLevelChanged::FDelegate FeatureLevelChangedDelegate = FOnFeatureLevelChanged::FDelegate::CreateRaw(this, &FOpenColorIOEditorModule::OnLevelEditorFeatureLevelChanged);
		FeatureLevelChangedDelegateHandle = EditorWorld->AddOnFeatureLevelChangedHandler(FeatureLevelChangedDelegate);
	}
}

void FOpenColorIOEditorModule::OnLevelEditorFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel)
{
	UOpenColorIOColorTransform::AllColorTransformsCacheResourceShadersForRendering();
}

void FOpenColorIOEditorModule::CleanFeatureLevelDelegate()
{
	if (FeatureLevelChangedDelegateHandle.IsValid())
	{
		UWorld* RegisteredWorld = EditorWorld.Get();
		if (RegisteredWorld)
		{
			RegisteredWorld->RemoveOnFeatureLevelChangedHandler(FeatureLevelChangedDelegateHandle);
		}

		FeatureLevelChangedDelegateHandle.Reset();
	}
}

void FOpenColorIOEditorModule::RegisterStyle()
{
#define IMAGE_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

	StyleInstance = MakeUnique<FSlateStyleSet>("OpenColorIOStyle");

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("OpenColorIO"));
	if (Plugin.IsValid())
	{
		StyleInstance->SetContentRoot(FPaths::Combine(Plugin->GetContentDir(), TEXT("Editor/Icons")));
	}

	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	StyleInstance->Set("ClassThumbnail.OpenColorIOConfiguration", new IMAGE_BRUSH_SVG("OpenColorIOConfiguration_64", Icon64x64));
	StyleInstance->Set("ClassIcon.OpenColorIOConfiguration", new IMAGE_BRUSH_SVG("OpenColorIOConfiguration_16", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());

#undef IMAGE_BRUSH
}

void FOpenColorIOEditorModule::UnregisterStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
	StyleInstance.Reset();
}

void FOpenColorIOEditorModule::RegisterViewMenuExtension()
{
	//Allows cleanup when module unloads
	FToolMenuOwnerScoped ToolMenuOwnerScoped(this);

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelViewportToolBar.View");
	FToolMenuSection& Section = Menu->FindOrAddSection(
		"ColorManagement", LOCTEXT("ColorManagement_Label", "Color Management"));
	Section.AddSubMenu("OCIODisplaySubMenu",
		LOCTEXT("OCIODisplaySubMenu_Label", "OCIO Display"),
		LOCTEXT("OCIODisplaySubMenu_ToolTip", "Configure the viewport to use an OCIO display configuration"),
		FNewToolMenuDelegate::CreateRaw(this, &FOpenColorIOEditorModule::AddOpenColorIODisplaySubMenu),
		false);
}

void FOpenColorIOEditorModule::UnregisterViewMenuExtension()
{
	if (UObjectInitialized() == true)
	{
		//Clean up pass
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
		UToolMenus::UnregisterOwner(this);
	}
}

void FOpenColorIOEditorModule::AddOpenColorIODisplaySubMenu(UToolMenu* Menu)
{
	// Disable searching in this parent menu because the contents of this menu is added as a
	// menu-in-a-menu which leads to double search fields if the parent is searchable too.
	Menu->bSearchable = false;

	FToolMenuSection& Section = Menu->AddDynamicSection("DynamicDisplayConfiguration",
		FNewToolMenuDelegate::CreateLambda([this](UToolMenu* DynamicSectionMenu) -> void {
			// Viewport menu was clicked, get which one was hit
			FViewport* CurrentViewport = GEditor->GetActiveViewport();

			// If no viewport has yet been set, we have nothing more to do.
			if (!CurrentViewport)
			{
				return;
			}

			// Make sure we know about that viewport
			TrackNewViewportIfRequired(CurrentViewport);

			// Fetch configuration for this viewport. If none were made, we'll populate UI with default values
			const FOpenColorIODisplayConfiguration& Configuration =
				IOpenColorIOModule::Get().GetDisplayManager().FindOrAddDisplayConfiguration(CurrentViewport->GetClient());

			// Add OCIO display section
			FToolMenuSection& Section = DynamicSectionMenu->AddSection(
				"DisplayConfiguration", LOCTEXT("DisplayConfiguration_Label", "Display Configuration"));
			Section.AddEntry(FToolMenuEntry::InitWidget("DisplayConfigurationWidget",
				SNew(SOpenColorIODisplay)
					.Viewport(CurrentViewport)
					.InitialConfiguration(Configuration)
					.OnConfigurationChanged(FOnDisplayConfigurationChanged::CreateRaw(
						this, &FOpenColorIOEditorModule::OnDisplayConfigurationChanged)),
				FText::GetEmpty(), // No Label
				true, // bNoIndent
				false)); // bSearchable
		}));
}

void FOpenColorIOEditorModule::OnDisplayConfigurationChanged(const FOpenColorIODisplayConfiguration& NewConfiguration)
{
	//Anytime a configuration option changes, update its values in the display manager to have the viewport state up to date
	FViewport* CurrentViewport = GEditor->GetActiveViewport();
	FOpenColorIODisplayConfiguration& Configuration = IOpenColorIOModule::Get().GetDisplayManager().FindOrAddDisplayConfiguration(CurrentViewport->GetClient());
	Configuration = NewConfiguration;
	CurrentViewport->Invalidate();
}

void FOpenColorIOEditorModule::OnLevelViewportClientListChanged()
{
	FOpenColorIODisplayManager& DisplayManager = IOpenColorIOModule::Get().GetDisplayManager();

	const TArray<FLevelEditorViewportClient*> LevelViewportClients = GEditor->GetLevelViewportClients();
	for (auto It = ConfiguredViewports.CreateIterator(); It; ++It)
	{
		FViewportPair& ViewportData = *It;

		//If a viewport we were tracking doesn't exist anymore
		// 1. Remove it from the collection if present
		// 2. Save its settings to config file for future session
		// 3. Remove from configured viewports
		if (!LevelViewportClients.ContainsByPredicate([ViewportData](const FLevelEditorViewportClient* Other) { return Other == ViewportData.Key; }))
		{
			const FOpenColorIODisplayConfiguration* Configuration = DisplayManager.GetDisplayConfiguration(ViewportData.Key);
			if (Configuration)
			{
				GetMutableDefault<UOpenColorIOLevelViewportSettings>()->SetViewportSettings(ViewportData.Value, *Configuration);
				GetMutableDefault<UOpenColorIOLevelViewportSettings>()->SaveConfig();
				DisplayManager.RemoveDisplayConfiguration(ViewportData.Key);
			}

			It.RemoveCurrent();
		}
	}

	// Verify if settings are present for new viewports not actually configured
	for (FLevelEditorViewportClient* LevelViewportClient : LevelViewportClients)
	{
		SetupViewportDisplaySettings(LevelViewportClient);
	}
}

void FOpenColorIOEditorModule::OnEngineLoopInitComplete()
{
	LLM_SCOPE_BYTAG(OpenColorIO_OpenColorIOEditor);
	// Register for Viewport updates to be able to track them
	GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FOpenColorIOEditorModule::OnLevelViewportClientListChanged);
}

void FOpenColorIOEditorModule::SetupViewportDisplaySettings(FLevelEditorViewportClient* Client)
{
	//If we haven't configured this viewport yet, it might be present in settings and we can apply its configuration
	if (!ConfiguredViewports.ContainsByPredicate([Client](const FViewportPair& Other) { return Other.Key == Client; }))
	{
		TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(Client->GetEditorViewportWidget());

		if (LevelViewport.IsValid()) // Might not be valid when exiting
		{
			const FOpenColorIODisplayConfiguration* PresetConfig = GetDefault<UOpenColorIOLevelViewportSettings>()->GetViewportSettings(LevelViewport->GetConfigKey());
			if (PresetConfig)
			{
				//Get display configuration for that viewport
				FOpenColorIODisplayConfiguration& ViewportConfig = IOpenColorIOModule::Get().GetDisplayManager().FindOrAddDisplayConfiguration(Client);

				//Write current settings so they become active
				ViewportConfig = *PresetConfig;

				//Since this viewport was preconfigured, add it to our list 
				FViewportPair NewEntry(Client, LevelViewport->GetConfigKey());
				ConfiguredViewports.Emplace(MoveTemp(NewEntry));
			}
		}
	}
}

void FOpenColorIOEditorModule::TrackNewViewportIfRequired(FViewport* Viewport)
{
	if (!ConfiguredViewports.ContainsByPredicate([Viewport](const FViewportPair& Other) { return Other.Key == Viewport->GetClient(); }))
	{
		//Find associated LevelViewport to get its identifier
		const TArray<FLevelEditorViewportClient*> LevelViewportClients = GEditor->GetLevelViewportClients();
		FLevelEditorViewportClient* const* AssociatedClient = LevelViewportClients.FindByPredicate([Viewport](const FLevelEditorViewportClient* Other) { return Other == Viewport->GetClient(); });

		if (AssociatedClient)
		{
			FLevelEditorViewportClient* Client = *AssociatedClient;
			TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(Client->GetEditorViewportWidget());
			if (LevelViewport.IsValid())
			{
				FViewportPair NewEntry(Client, LevelViewport->GetConfigKey());
				ConfiguredViewports.Emplace(MoveTemp(NewEntry));
			}
		}
	}
}

IMPLEMENT_MODULE(FOpenColorIOEditorModule, OpenColorIOEditor);

#undef LOCTEXT_NAMESPACE
