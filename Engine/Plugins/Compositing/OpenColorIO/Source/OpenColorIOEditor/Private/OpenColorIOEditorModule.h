// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenColorIOEditorModule.h"

#include "CoreMinimal.h"
#include "Engine/World.h"


class IAssetTypeActions;
class FLevelEditorViewportClient;
class FSlateStyleSet;
class FViewport;
class UToolMenu;
struct FOpenColorIODisplayConfiguration;

/**
 * Implements the OpenColorIOEditor module.
 */
class FOpenColorIOEditorModule : public IOpenColorIOEditorModule, public TSharedFromThis<FOpenColorIOEditorModule>
{
public:

	virtual ~FOpenColorIOEditorModule() = default;

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** Set a color transform display configuration on the active editor viewport. */
	void SetActiveViewportConfiguration(const FOpenColorIODisplayConfiguration& InConfiguration);

protected:
	void RegisterCustomizations();
	void UnregisterCustomizations();
	void RegisterSettings();
	void UnregisterSettings();

	void OnWorldInit(UWorld* InWorld, const UWorld::InitializationValues InInitializationValues);
	void OnLevelEditorFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel);
	void CleanFeatureLevelDelegate();
	void RegisterStyle();
	void UnregisterStyle();

	/** Registers and configure display look configuration in the View menu */
	void RegisterViewMenuExtension();
	void UnregisterViewMenuExtension();
	void AddOpenColorIODisplaySubMenu(UToolMenu* Menu);

	/** Callback when a configuration has changed to update the display manager of a new configuration */
	void OnDisplayConfigurationChanged(const FOpenColorIODisplayConfiguration& NewConfiguration);

	/** Callback when Level viewport list changed to be able to remove a configured one */
	void OnLevelViewportClientListChanged();

	/** Callback when engine loop is done to be able to register ourselves to LevelViewport when editor is valid. */
	void OnEngineLoopInitComplete();

	/** Verify in settings if a viewport's identifier is pre-configured and add it if yes */
	void SetupViewportDisplaySettings(FLevelEditorViewportClient* Client);

	/** Creates a new ViewportData entry if none exist for the given one */
	void TrackNewViewportIfRequired(FViewport* Viewport);

private:

	TWeakObjectPtr<UWorld> EditorWorld;
	FDelegateHandle FeatureLevelChangedDelegateHandle;
	TUniquePtr<FSlateStyleSet> StyleInstance;
	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	/** List of existing viewport we configured to have display look with associated identifier */
	using FViewportPair = TPair<FLevelEditorViewportClient*, FName>;
	TArray<FViewportPair> ConfiguredViewports;
};

