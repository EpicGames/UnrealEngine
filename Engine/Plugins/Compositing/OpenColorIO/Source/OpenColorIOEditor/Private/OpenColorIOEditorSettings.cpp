// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOEditorSettings.h"

#include "IOpenColorIOEditorModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "OpenColorIOSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenColorIOEditorSettings)

const FOpenColorIODisplayConfiguration* UOpenColorIOLevelViewportSettings::GetViewportSettings(FName ViewportIdentifier) const
{
	const FPerViewportDisplaySettingPair* Pair = ViewportsSettings.FindByPredicate([ViewportIdentifier](const FPerViewportDisplaySettingPair& Other)
		{
			return Other.ViewportIdentifier == ViewportIdentifier;
		});

	if (Pair)
	{
		return &Pair->DisplayConfiguration;
	}

	const UOpenColorIODefaultViewportSettings* DefaultViewportSettings = GetDefault<UOpenColorIODefaultViewportSettings>();
	
	if (DefaultViewportSettings->DefaultDisplayConfiguration.bIsEnabled)
	{
		// Apply default viewport settings if there is no locally cached viewport settings.
		return &DefaultViewportSettings->DefaultDisplayConfiguration;
	}

	return nullptr;
}

void UOpenColorIOLevelViewportSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Note: This config could be moved back to (config = EditorPerProjectUserSettings) since level viewport layouts are local & user specific.

	if (GConfig && FPaths::FileExists(GEditorPerProjectIni))
	{
		const TCHAR* SectionName = TEXT("/Script/OpenColorIOEditor.OpenColorIOLevelViewportSettings");

		if (GConfig->DoesSectionExist(SectionName, GEditorPerProjectIni))
		{
			LoadConfig(UOpenColorIOLevelViewportSettings::StaticClass(), *GEditorPerProjectIni);

			SaveConfig();

			GConfig->EmptySection(SectionName, *GEditorPerProjectIni);

			UE_LOGF(LogOpenColorIOEditor, Warning, "Migrated EditorPerProjectUserSettings OpenColorIO settings to plugin-specific config file: %ls.", *GetClass()->GetConfigName());
		}
	}

	// Discard disabled empty/invalid settings
	ViewportsSettings.RemoveAll([](const FPerViewportDisplaySettingPair& ViewportSetting)
		{
			return !ViewportSetting.DisplayConfiguration.bIsEnabled && !ViewportSetting.DisplayConfiguration.ColorConfiguration.IsValid();
		}
	);

	const bool bEnforceForwardViewDirectionOnly = !GetDefault<UOpenColorIOSettings>()->bSupportInverseViewTransforms;
	if (bEnforceForwardViewDirectionOnly)
	{
		for (FPerViewportDisplaySettingPair& ViewportSetting : ViewportsSettings)
		{
			ViewportSetting.DisplayConfiguration.ColorConfiguration.DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;
		}
	}
}

void UOpenColorIOLevelViewportSettings::SetViewportSettings(FName ViewportIdentifier, const FOpenColorIODisplayConfiguration& Configuration)
{
	auto FindMatchingViewportFn = [ViewportIdentifier](const FPerViewportDisplaySettingPair& Other)
		{
			return Other.ViewportIdentifier == ViewportIdentifier;
		};

	const bool bRemoveConfiguration = !Configuration.bIsEnabled && !Configuration.ColorConfiguration.IsValid();
	const UOpenColorIODefaultViewportSettings* DefaultViewportSettings = GetDefault<UOpenColorIODefaultViewportSettings>();

	if (bRemoveConfiguration || DefaultViewportSettings->DefaultDisplayConfiguration.Equals(Configuration))
	{
		// Remove settings if they are disabled & invalid, or if they match the default viewport configuration
		ViewportsSettings.RemoveAll(FindMatchingViewportFn);
	}
	else if (FPerViewportDisplaySettingPair* Pair = ViewportsSettings.FindByPredicate(FindMatchingViewportFn))
	{
		Pair->DisplayConfiguration = Configuration;
	}
	else
	{
		//Add new entry if viewport is not found
		FPerViewportDisplaySettingPair NewEntry;
		NewEntry.ViewportIdentifier = ViewportIdentifier;
		NewEntry.DisplayConfiguration = Configuration;
		ViewportsSettings.Emplace(MoveTemp(NewEntry));
	}
}

void UOpenColorIODefaultViewportSettings::PostInitProperties()
{
	Super::PostInitProperties();

	const bool bEnforceForwardViewDirectionOnly = !GetDefault<UOpenColorIOSettings>()->bSupportInverseViewTransforms;
	if (bEnforceForwardViewDirectionOnly)
	{
		DefaultDisplayConfiguration.ColorConfiguration.DisplayViewDirection = EOpenColorIOViewTransformDirection::Forward;
	}
}
