// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"

#include "OpenColorIOEditorSettings.generated.h"

class FViewport;
class SOpenColorIOColorSpacePicker;
class SWidget;
class UToolMenu;
struct FAssetData;


USTRUCT()
struct FPerViewportDisplaySettingPair
{
	GENERATED_BODY()

	/*  Name associated with this viewport's layout to identify it. */
	UPROPERTY(config)
	FName ViewportIdentifier;

	/* Display configuration for a given viewport */
	UPROPERTY(config)
	FOpenColorIODisplayConfiguration DisplayConfiguration;
};

/**
 * List of settings associated to level viewport instances linked with an identifier
 */
UCLASS(config = OpenColorIO)
class UOpenColorIOLevelViewportSettings : public UObject
{
	GENERATED_BODY()

public:

	virtual void PostInitProperties() override;

	/** Returns setting associated with a given viewport identifier */
	const FOpenColorIODisplayConfiguration* GetViewportSettings(FName ViewportIdentifier) const;

	/** Add or Update settings for a given viewport identifier */
	void SetViewportSettings(FName ViewportIdentifier, const FOpenColorIODisplayConfiguration& Configuration);

protected:

	/** Settings associated to each viewport that was configured */
	UPROPERTY(config)
	TArray<FPerViewportDisplaySettingPair> ViewportsSettings;
};

UCLASS(config = OpenColorIO)
class UOpenColorIODefaultViewportSettings : public UObject
{
	GENERATED_BODY()

public:

	virtual void PostInitProperties() override;

	/** Default settings applied to level viewports when none was previously set, or if settings have been cleared. */
	UPROPERTY(config, EditAnywhere, Category = OpenColorIO, DisplayName="Default Viewport Configuration")
	FOpenColorIODisplayConfiguration DefaultDisplayConfiguration;
};
