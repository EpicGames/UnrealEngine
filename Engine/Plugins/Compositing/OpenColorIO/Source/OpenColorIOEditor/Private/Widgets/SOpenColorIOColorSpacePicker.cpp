// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOpenColorIOColorSpacePicker.h"

#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOSettings.h"
#include "SResetToDefaultMenu.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SOpenColorIOColorPicker"

void SOpenColorIOColorSpacePicker::Construct(const FArguments& InArgs)
{
	Configuration = InArgs._Config;
	bIsDestination = InArgs._IsDestination;
	Selection = InArgs._Selection;
	SelectionRestriction = InArgs._SelectionRestriction;
	OnColorSpaceChanged = InArgs._OnColorSpaceChanged;

	SelectionButton = SNew(SComboButton)
		.OnGetMenuContent_Lambda([this]() { return HandleColorSpaceComboButtonMenuContent(); })
		.ContentPadding(FMargin(4.0, 2.0));

	ChildSlot
		[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(MakeAttributeLambda([this]
				{
					if (Selection.IsSet() && !Selection.Get().IsEmpty())
					{
						return Selection.Get();
					}
					
					return LOCTEXT("None", "<None>");
				}))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SelectionButton.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		[
			SNew(SButton)
			.ContentPadding(0)
			.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
			.ButtonStyle(FAppStyle::Get(), "ToggleButton") 
			.OnClicked(this, &SOpenColorIOColorSpacePicker::OnResetToDefault)
			.Visibility(this, &SOpenColorIOColorSpacePicker::ShouldShowResetToDefaultButton)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
			]
		]
	];
}

void SOpenColorIOColorSpacePicker::SetConfiguration(TWeakObjectPtr<UOpenColorIOConfiguration> NewConfiguration)
{
	Configuration = NewConfiguration;
}

void SOpenColorIOColorSpacePicker::SetCurrentColorSpace(const FOpenColorIOColorSpace& NewColorSpace)
{
	// Let listeners know selection has changed
	OnColorSpaceChanged.ExecuteIfBound(NewColorSpace, {}, bIsDestination);

	//Close our menu
	if (SelectionButton.IsValid())
	{
		SelectionButton->SetIsOpen(false);
	}
}

void SOpenColorIOColorSpacePicker::SetCurrentDisplayView(const FOpenColorIODisplayView& NewDisplayView)
{
	// Let listeners know selection has changed
	OnColorSpaceChanged.ExecuteIfBound({}, NewDisplayView, bIsDestination);

	//Close our menu
	if (SelectionButton.IsValid())
	{
		SelectionButton->SetIsOpen(false);
	}
}

TSharedRef<SWidget> SOpenColorIOColorSpacePicker::HandleColorSpaceComboButtonMenuContent()
{
	if (UOpenColorIOConfiguration* ConfigurationObject = Configuration.Get())
	{
		// generate menu
		const bool bShouldCloseWindowAfterClosing = false;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterClosing, nullptr);

		MenuBuilder.BeginSection("AvailableColorSpaces", LOCTEXT("AvailableColorSpaces", "Available Color Spaces"));
		{
			bool ColorSpaceAdded = false;

			for (int32 i = 0; i < ConfigurationObject->DesiredColorSpaces.Num(); ++i)
			{
				const FOpenColorIOColorSpace& ColorSpace = ConfigurationObject->DesiredColorSpaces[i];
				if (!ColorSpace.IsValid())
				{
					continue;
				}

				const FString ColorSpaceStr = ColorSpace.ToString();
				if (ColorSpaceStr == SelectionRestriction.Get())
				{
					continue;
				}

				MenuBuilder.AddMenuEntry
				(
					FText::FromString(ColorSpaceStr),
					FText::FromString(ColorSpace.Description),
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateLambda([this, ColorSpace]
						{
							SetCurrentColorSpace(ColorSpace);
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([this, ColorSpaceStr]
						{
							return Selection.Get().ToString() == ColorSpaceStr;
						})
						),
						NAME_None,
						EUserInterfaceActionType::RadioButton
					);

				ColorSpaceAdded = true;
			}

			if (!ColorSpaceAdded)
			{
				MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoColorSpaceFound", "No available color spaces"), false, false);
			}
		}
		MenuBuilder.EndSection();

		if (bIsDestination)
		{
			MenuBuilder.BeginSection("AvailableDisplayViews", LOCTEXT("AvailableDisplayViews", "Available Display-Views"));
			{
				bool DisplayViewAdded = false;

				for (int32 i = 0; i < ConfigurationObject->DesiredDisplayViews.Num(); ++i)
				{
					const FOpenColorIODisplayView& DisplayView = ConfigurationObject->DesiredDisplayViews[i];
					if (!DisplayView.IsValid())
					{
						continue;
					}
					
					const FString DisplayViewStr = DisplayView.ToString();
					if (DisplayViewStr == SelectionRestriction.Get())
					{
						continue;
					}

					MenuBuilder.AddMenuEntry
					(
						FText::FromString(DisplayViewStr),
						FText::FromString(DisplayView.Description),
						FSlateIcon(),
						FUIAction
						(
							FExecuteAction::CreateLambda([this, DisplayView]
							{
								SetCurrentDisplayView(DisplayView);
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, DisplayViewStr]
							{
								return Selection.Get().ToString() == DisplayViewStr;
							})
						),
						NAME_None,
						EUserInterfaceActionType::RadioButton
					);

					DisplayViewAdded = true;
				}

				if (!DisplayViewAdded)
				{
					MenuBuilder.AddWidget(SNullWidget::NullWidget, LOCTEXT("NoDisplayViewFound", "No available display-view"), false, false);
				}
			}
			MenuBuilder.EndSection();
		}


		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

FReply SOpenColorIOColorSpacePicker::OnResetToDefault()
{
	// Let listeners know selection has changed
	OnColorSpaceChanged.ExecuteIfBound({}, {}, bIsDestination);

	//Close our menu
	if (SelectionButton.IsValid())
	{
		SelectionButton->SetIsOpen(false);
	}

	return FReply::Handled();
}

EVisibility SOpenColorIOColorSpacePicker::ShouldShowResetToDefaultButton() const
{
	return (Selection.IsSet() && !Selection.Get().IsEmpty()) ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
