// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "OpenColorIOColorSpace.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class FViewportClient;
class SComboButton;
class UOpenColorIOConfiguration;

DECLARE_DELEGATE_ThreeParams(FOnColorSpaceChanged, const FOpenColorIOColorSpace& /*ColorSpace*/, const FOpenColorIODisplayView& /*DisplayView*/, bool /*bIsDestination*/);

/** Pair of OpenColorIO transform selection objects. */
struct FOpenColorIOPickerSelection
{
	FOpenColorIOColorSpace ColorSpace;
	FOpenColorIODisplayView DisplayView;
};

/** OpenColorIO transformation source or destination. */
enum EOpenColorIOTransformDomain : uint32
{
	OCIO_Src = 0,
	OCIO_Dst = 1,
};

class SOpenColorIOColorSpacePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOpenColorIOColorSpacePicker) {}
		SLATE_ARGUMENT(TWeakObjectPtr<UOpenColorIOConfiguration>, Config)
		SLATE_ARGUMENT(bool, IsDestination)
		SLATE_ATTRIBUTE(FText, Selection)
		SLATE_ATTRIBUTE(FString, SelectionRestriction)
		SLATE_EVENT(FOnColorSpaceChanged, OnColorSpaceChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	/** Update current configuration asset for this picker */
	void SetConfiguration(TWeakObjectPtr<UOpenColorIOConfiguration> NewConfiguration);

protected:
	
	/** Called when a selection has been made */
	void SetCurrentColorSpace(const FOpenColorIOColorSpace& NewColorSpace);
	void SetCurrentDisplayView(const FOpenColorIODisplayView& NewDisplayView);

	/** Handles color space list menu creation */
	TSharedRef<SWidget> HandleColorSpaceComboButtonMenuContent();
	
	/** Reset to default triggered in UI */
	FReply OnResetToDefault();

	/** Whether or not ResetToDefault button should be shown */
	EVisibility ShouldShowResetToDefaultButton() const;

protected:
	TSharedPtr<SComboButton> SelectionButton;
	TWeakObjectPtr<UOpenColorIOConfiguration> Configuration;
	TAttribute<FText> Selection;
	TAttribute<FString> SelectionRestriction;
	FOnColorSpaceChanged OnColorSpaceChanged;
	bool bIsDestination = false;
};
