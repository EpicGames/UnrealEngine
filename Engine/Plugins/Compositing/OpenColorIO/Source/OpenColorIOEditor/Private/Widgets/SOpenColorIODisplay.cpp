// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOpenColorIODisplay.h"

#include "AssetRegistry/AssetData.h"
#include "Framework/Application/SlateApplication.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "SOpenColorIOColorSpacePicker.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Input/SCheckBox.h"


#define LOCTEXT_NAMESPACE "SOpenColorIODisplay"


namespace OpenColorIODisplayMenuUtils
{
	static const FName DisplayConfigMenuName("OpenColorIODisplay.DisplayConfiguration");
}

void SOpenColorIODisplay::Construct(const FArguments& InArgs)
{
	Viewport = InArgs._Viewport;
	Configuration = InArgs._InitialConfiguration;
	OnDisplayConfigurationChanged = InArgs._OnConfigurationChanged;
	check(Viewport);

	// Register configuration menu if it hasn't been yet
	UToolMenu* Menu = UToolMenus::Get()->FindMenu(OpenColorIODisplayMenuUtils::DisplayConfigMenuName);
	if (Menu == nullptr)
	{
		Menu = UToolMenus::Get()->RegisterMenu(OpenColorIODisplayMenuUtils::DisplayConfigMenuName);
	}

	check(Menu);

	// Add section to enable/disable the display look
	FToolMenuSection& Section = Menu->AddSection("DisplayConfiguration", FText::GetEmpty());
	Section.AddMenuEntry(
		"EnableDisplay",
		LOCTEXT("EnableDisplay_Label", "Enable Display"),
		LOCTEXT("EnableDisplay_ToolTip", "Toggle Display color conversion"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &SOpenColorIODisplay::ToggleEnableDisplay),
			FCanExecuteAction::CreateRaw(this, &SOpenColorIODisplay::CanEnableDisplayConfiguration),
			FIsActionChecked::CreateRaw(this, &SOpenColorIODisplay::GetDisplayConfigurationState)),
		EUserInterfaceActionType::ToggleButton);

	// Add a section to select the OCIO configuration asset from which color spaces will be taken from
	Section.AddEntry(FToolMenuEntry::InitSubMenu(
		"OCIOAssetSelection",
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SOpenColorIODisplay::GetConfigurationText)),
		LOCTEXT("OCIOAssetSelection_Tooltip", "Select the asset from which color spaces will be fetched"),
		FNewToolMenuDelegate::CreateRaw(this, &SOpenColorIODisplay::SelectConfigSubMenu),
		FUIAction(),
		EUserInterfaceActionType::None
	));

	// Source color space picker widget
	TransformPicker[OCIO_Src] =
		SNew(SOpenColorIOColorSpacePicker)
		.Config(Configuration.ColorConfiguration.ConfigurationSource)
		.Selection_Lambda([this]() { return FText::FromString(Configuration.ColorConfiguration.GetSourceString()); })
		.SelectionRestriction_Lambda([this]() { return Configuration.ColorConfiguration.GetDestinationString(); })
		.IsDestination(false)
		.OnColorSpaceChanged(FOnColorSpaceChanged::CreateSP(this, &SOpenColorIODisplay::OnSelectionChanged));

	// Destination color space picker widget
	TransformPicker[OCIO_Dst] =
		SNew(SOpenColorIOColorSpacePicker)
		.Config(Configuration.ColorConfiguration.ConfigurationSource)
		.Selection_Lambda([this]() { return FText::FromString(Configuration.ColorConfiguration.GetDestinationString()); })
		.SelectionRestriction_Lambda([this]() { return Configuration.ColorConfiguration.GetSourceString(); })
		.IsDestination(true)
		.OnColorSpaceChanged(FOnColorSpaceChanged::CreateSP(this, &SOpenColorIODisplay::OnSelectionChanged));
	
	// Note: FText::GetEmpty() would previously result in incorrect indentation.
	Section.AddEntry(FToolMenuEntry::InitWidget("SourceColor", TransformPicker[OCIO_Src].ToSharedRef(), FText::FromString(TEXT(" ")), false, false));
	Section.AddEntry(FToolMenuEntry::InitWidget("DestinationColor", TransformPicker[OCIO_Dst].ToSharedRef(), FText::FromString(TEXT(" ")), false, false));
	
	const UOpenColorIOSettings* Settings = GetDefault<UOpenColorIOSettings>();
	if (Settings->bSupportInverseViewTransforms)
	{
		TSharedRef<SWidget> InvertCheckbox = SNew(SCheckBox)
			.IsChecked_Lambda([this]() -> ECheckBoxState
			{
				if (Configuration.ColorConfiguration.IsDisplayView())
				{
					return Configuration.ColorConfiguration.DisplayViewDirection == EOpenColorIOViewTransformDirection::Inverse ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				
				return ECheckBoxState::Undetermined;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				Configuration.ColorConfiguration.DisplayViewDirection = (NewState == ECheckBoxState::Checked) ? EOpenColorIOViewTransformDirection::Inverse : EOpenColorIOViewTransformDirection::Forward;

				OnDisplayConfigurationChanged.ExecuteIfBound(Configuration);
			})
			.IsEnabled_Lambda([this]()
			{
				return Configuration.ColorConfiguration.IsDisplayView();
			});

		Section.AddEntry(
			FToolMenuEntry::InitWidget(
				"InvertViewTransform",
				InvertCheckbox,
				LOCTEXT("InvertViewTransform", "Invert View Transform")
			));
	}


	TSharedRef<SWidget> Widget = UToolMenus::Get()->GenerateWidget(Menu);
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0)
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.f, 0.f, 4.f, 0.f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					Widget
				]
			]
		];
}

FText SOpenColorIODisplay::GetConfigurationText() const
{
	if (Configuration.ColorConfiguration.ConfigurationSource)
	{
		return FText::FromName(Configuration.ColorConfiguration.ConfigurationSource->GetFName());
	}
	else
	{
		return LOCTEXT("OCIOAssetSelection_Label", "Select an OCIO Asset");
	}
}

void SOpenColorIODisplay::SelectConfigSubMenu(UToolMenu* Menu)
{
	FAssetData CurrentAssetData = Configuration.ColorConfiguration.ConfigurationSource ? FAssetData(Configuration.ColorConfiguration.ConfigurationSource) : FAssetData();

	TArray<const UClass*> ClassFilters;
	ClassFilters.Add(UOpenColorIOConfiguration::StaticClass());

	FToolMenuSection& AssetSelectionMenu = Menu->AddSection("OCIOAssetSelection", LOCTEXT("OCIOAssetSelection_GenericLabel", "Asset selection"));
	ClassPicker = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
		CurrentAssetData,
		Configuration.ColorConfiguration.ConfigurationSource != nullptr,
		false,
		ClassFilters,
		TArray<UFactory*>(),
		FOnShouldFilterAsset::CreateLambda([CurrentAssetData](const FAssetData& InAssetData) { return InAssetData == CurrentAssetData; }),
		FOnAssetSelected::CreateSP(this, &SOpenColorIODisplay::OnConfigSelected),
		FSimpleDelegate()
	);
	
	AssetSelectionMenu.AddEntry(FToolMenuEntry::InitWidget("OCIOAssetPickerWidget",
		ClassPicker.ToSharedRef(),
		FText::GetEmpty(),
		true, //NoIndent
		false)); //Searchable
}

void SOpenColorIODisplay::OnConfigSelected(const FAssetData& AssetData)
{
	FSlateApplication::Get().DismissMenuByWidget(ClassPicker.ToSharedRef());

	UOpenColorIOConfiguration* NewConfig = Cast<UOpenColorIOConfiguration>(AssetData.GetAsset());
	
	Configuration.ColorConfiguration.Reset();
	Configuration.ColorConfiguration.ConfigurationSource = NewConfig;
	TransformPicker[OCIO_Src]->SetConfiguration(NewConfig);
	TransformPicker[OCIO_Dst]->SetConfiguration(NewConfig);

	OnDisplayConfigurationChanged.ExecuteIfBound(Configuration);
}

void SOpenColorIODisplay::ToggleEnableDisplay()
{
	Configuration.bIsEnabled = !Configuration.bIsEnabled;

	OnDisplayConfigurationChanged.ExecuteIfBound(Configuration);
}

bool SOpenColorIODisplay::CanEnableDisplayConfiguration()
{
	//For now, always allow it to be enabled. We could add filters based on current view modes
	return true;
}

bool SOpenColorIODisplay::GetDisplayConfigurationState()
{
	return Configuration.bIsEnabled;
}

void SOpenColorIODisplay::OnSelectionChanged(const FOpenColorIOColorSpace& NewColorSpace, const FOpenColorIODisplayView& NewDisplayView, bool bIsDestination)
{
	FOpenColorIOColorConversionSettings& ConversionSettings = Configuration.ColorConfiguration;

	if (bIsDestination)
	{
		if (NewDisplayView.IsValid())
		{
			ConversionSettings.DestinationColorSpace.Reset();
			ConversionSettings.DestinationDisplayView = NewDisplayView;
		}
		else
		{
			ConversionSettings.DestinationDisplayView.Reset();
			ConversionSettings.DestinationColorSpace = NewColorSpace;
		}
	}
	else
	{
		ConversionSettings.SourceColorSpace = NewColorSpace;
	}

	OnDisplayConfigurationChanged.ExecuteIfBound(Configuration);
}

#undef LOCTEXT_NAMESPACE
