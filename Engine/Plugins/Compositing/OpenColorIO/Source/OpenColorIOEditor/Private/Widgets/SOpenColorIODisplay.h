// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"
#include "SOpenColorIOColorSpacePicker.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class FViewport;
class SWidget;
class UToolMenu;
struct FAssetData;


DECLARE_DELEGATE_OneParam(FOnDisplayConfigurationChanged, const FOpenColorIODisplayConfiguration& /*Configuration*/);


class SOpenColorIODisplay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOpenColorIODisplay) {}
		SLATE_ARGUMENT(FViewport*, Viewport)
		SLATE_ARGUMENT(FOpenColorIODisplayConfiguration, InitialConfiguration)
		SLATE_EVENT(FOnDisplayConfigurationChanged, OnConfigurationChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

protected:
	
	FText GetConfigurationText() const;
	void SelectConfigSubMenu(UToolMenu* Menu);
	void OnConfigSelected(const FAssetData& AssetData);
	void ToggleEnableDisplay();
	bool CanEnableDisplayConfiguration();
	bool GetDisplayConfigurationState();
	void OnSelectionChanged(const FOpenColorIOColorSpace& NewColorSpace, const FOpenColorIODisplayView& NewDisplayView, bool bIsDestination);
	
protected:

	FViewport* Viewport;

	/**Class picker widget to be able to close back the menu once selection has been made */
	TSharedPtr<SWidget> ClassPicker;

	/** ColorSpace pickers reference to update them when config asset is changed */
	TStaticArray<TSharedPtr<SOpenColorIOColorSpacePicker>, 2> TransformPicker;

	/** Current configuration */
	FOpenColorIODisplayConfiguration Configuration;

	/** Callback triggered any time the configuration changed */
	FOnDisplayConfigurationChanged OnDisplayConfigurationChanged;
};
