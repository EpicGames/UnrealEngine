// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"


DECLARE_LOG_CATEGORY_EXTERN(LogOpenColorIOEditor, Log, All);

/**
 * Interface for the OpenColorIOEditor module.
 */
class IOpenColorIOEditorModule : public IModuleInterface
{
public:
};

