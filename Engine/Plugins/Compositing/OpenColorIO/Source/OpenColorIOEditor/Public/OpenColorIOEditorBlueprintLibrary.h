// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/TextureDefines.h"

#include "OpenColorIOEditorBlueprintLibrary.generated.h"

struct FOpenColorIODisplayConfiguration;

UCLASS()
class UOpenColorIOEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Set the active editor viewport's display configuration color transform .
	 *
	 * @param InDisplayConfiguration Display configuration color transform
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	static OPENCOLORIOEDITOR_API void SetActiveViewportConfiguration(const FOpenColorIODisplayConfiguration& InConfiguration);

	/**
	 * Apply a color space transform to a color value.
	 *
	 * @param ConversionSettings Color transformation settings.
	 * @param InOutColor Color value transform.
	 * @return true upon success.
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	static OPENCOLORIOEDITOR_API bool ApplyColorSpaceTransformToColor(const FOpenColorIOColorConversionSettings& ConversionSettings, const FLinearColor& InColor, FLinearColor& OutColor);

	/**
	 * Apply a color space transform to a texture asset.
	 *
	 * @param ConversionSettings Color transformation settings.
	 * @param InOutTexture Texture object to transform.
	 * @return true upon success.
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	static OPENCOLORIOEDITOR_API bool ApplyColorSpaceTransformToTexture(const FOpenColorIOColorConversionSettings& ConversionSettings, UTexture* InOutTexture);

	/**
	 * Apply a color space transform with a target compression setting to a texture asset.
	 *
	 * @param ConversionSettings Color transformation settings.
	 * @param TargetCompression Target texture compression setting.
	 * @param InOutTexture Texture object to transform.
	 * @return true upon success.
	 */
	UFUNCTION(BlueprintCallable, Category = "OpenColorIO")
	static OPENCOLORIOEDITOR_API bool ApplyColorSpaceTransformToTextureCompressed(const FOpenColorIOColorConversionSettings& ConversionSettings, TextureCompressionSettings TargetCompression, UTexture* InOutTexture);
};
