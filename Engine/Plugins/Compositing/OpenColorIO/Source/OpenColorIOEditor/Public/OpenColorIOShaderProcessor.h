// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"

#include "OpenColorIOShaderCompilationManager.h"


// Handles ticking the shader compilation manager and dispatch the results
//
class FOpenColorIOShaderProcessorTickable : FTickableEditorObject
{
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual void Tick(float DeltaSeconds) override
	{
		GOpenColorIOShaderCompilationManager.Tick(DeltaSeconds);
		GOpenColorIOShaderCompilationManager.ProcessAsyncResults();
	}

	virtual TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FOpenColorIOShaderProcessorTickable, STATGROUP_Tickables);
	}
};